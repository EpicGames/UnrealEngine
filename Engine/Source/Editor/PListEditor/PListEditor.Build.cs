// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PListEditor : ModuleRules
{
	public PListEditor( ReadOnlyTargetRules Target ) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"InputCore",
				"Slate",
				"SlateCore",
				"DesktopPlatform",
				"XmlParser",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"WorkspaceMenuStructure",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
			}
		);
	}
}
