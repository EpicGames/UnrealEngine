// Copyright Epic Games, Inc. All Rights Reserved.

#include "PListEditor.h"

#include "Delegates/Delegate.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "SPlistEditor.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"

IMPLEMENT_MODULE( FPListEditor, PListEditor );

const FName PListEditorApp = FName(TEXT("PListEditorApp"));

TSharedRef<SDockTab> CreatePListEditorTab( const FSpawnTabArgs& Args )
{
	TSharedPtr<SPListEditorPanel> EditorPanel;

	return SNew(SDockTab)
		.TabRole( ETabRole::NomadTab )
		.Label( NSLOCTEXT("PListEditorApp", "TabTitle", "PList Editor") )
		.OnCanCloseTab(EditorPanel.ToSharedRef(), &SPListEditorPanel::OnTabClose)
		.Content()
		[
			SAssignNew(EditorPanel, SPListEditorPanel)
		];
}

void FPListEditor::StartupModule()
{
	// Create tab spawner
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner( PListEditorApp, FOnSpawnTab::CreateStatic( &CreatePListEditorTab ) )
		.SetDisplayName( NSLOCTEXT("PListEditorApp", "TabTitle", "PList Editor") )
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());

	// Register Commands
	FPListEditorCommands::Register();
}

void FPListEditor::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner( PListEditorApp );
	}
}
