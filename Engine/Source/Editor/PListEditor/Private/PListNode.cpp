// Copyright Epic Games, Inc. All Rights Reserved.

#include "PListNode.h"

#include "Containers/BitArray.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

/** A default function to return an invalid row widget */
TSharedRef<ITableRow> GenerateInvalidRow(const TSharedRef<STableViewBase>& OwnerTable, const FText& ErrorMessage)
{
	return
	SNew(STableRow<TSharedPtr<ITableRow> >, OwnerTable)
	.Content()
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 1)
		[
			SNew(STextBlock)
			.Text(ErrorMessage)
		]
	];
}

/** A default function to return an invalid row widget with columns */
TSharedRef<SWidget> GenerateInvalidRow(const FText& ErrorMessage)
{
	return
	SNew(SHorizontalBox)

	+SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2, 1)
	[
		SNew(STextBlock)
		.Text(ErrorMessage)
	];
}

/** Helper function to generate an FString with the specified number of tabs */
FString GenerateTabString(int32 NumTabs)
{
	check(NumTabs >= 0);

	FString ToRet;
	for(int32 i = 0; i < NumTabs; ++i)
	{
		ToRet += TEXT("\t");
	}
	return ToRet;
}

/** Sets the KeyString of the node, needed for telling children to set their keys. Default do nothing. */
void IPListNode::SetKey(FString NewKey)
{
}

/** Sets the value of the node. Default do nothing */
void IPListNode::SetValue(FString NewValue)
{
}

/** Sets the value of the node. Default do nothing */
void IPListNode::SetValue(bool bNewValue)
{
}

/** Sets a flag denoting whether the element is in an array. Default do nothing */
void IPListNode::SetArrayMember(bool bArrayMember)
{
}

/** Gets the overlay brush dynamically */
const FSlateBrush* IPListNode::GetOverlayBrush()
{
	return nullptr;
}

/** Checks if a key string is valid */
bool IPListNode::IsKeyStringValid(FString ToCheck)
{
	return ToCheck.Len() > 0;
}

/** Checks if a value string is valid */
bool IPListNode::IsValueStringValid(FString ToCheck)
{
	return ToCheck.Len() > 0;
}

/** Delegate: Gets the overlay brush from derived children classes */
const FSlateBrush* IPListNode::GetOverlayBrushDelegate(const TSharedRef<class IPListNode> In)
{
	return In->GetOverlayBrush();
}

/** Sets the depth of the node */
void IPListNode::SetDepth(int32 InDepth)
{
	Depth = InDepth;
}

/** Gets the depth of the node */
int32 IPListNode::GetDepth()
{
	return Depth;
}
