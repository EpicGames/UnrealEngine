// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FText;
class ITableRow;
class SPListEditorPanel;
class STableViewBase;
class SWidget;
struct FSlateBrush;

/**
 * Represents the types of data that are supported by plist files.
 */
enum EPLNTypes
{
	PLN_String,
	PLN_Boolean,
	//PLN_Real,
	//PLN_Integer,
	//PLN_Date,
	//PLN_Data,
	PLN_File,
	PLN_Dictionary,
	PLN_Array
};

/** A default function to return an invalid row widget */
TSharedRef<ITableRow> GenerateInvalidRow(const TSharedRef<STableViewBase>& OwnerTable, const FText& ErrorMessage);
/** A default function to return an invalid row widget with columns */
TSharedRef<SWidget> GenerateInvalidRow(const FText& ErrorMessage);
/** Helper function to generate an FString with the specified number of tabs */
FString GenerateTabString(int32 NumTabs);

/** An interface to a PListNode that can be stored in an STreeView */
class IPListNode : public TSharedFromThis<IPListNode>
{
public:
	IPListNode(SPListEditorPanel* InEditorWidget)
		: EditorWidget(InEditorWidget), Depth(-1)
	{}

public:

	/** No base class creations */
	virtual ~IPListNode() = default;

	/** Validation check */
	virtual bool IsValid() = 0;

	/** Returns the array of children. Note: the returned array is modifyable */
	virtual TArray<TSharedPtr<IPListNode> >& GetChildren() = 0;

	/** Adds a child to the internal array of the class */
	virtual void AddChild(TSharedPtr<IPListNode> InChild) = 0;

	/** Gets the type of the node via enum */
	virtual EPLNTypes GetType() = 0;

	/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
	virtual bool UsesColumns() = 0;

	/** Generates a widget for a TableView row */
	/** Note: Uses the default expansion button but we want our own since default doesn't work with columns */
	virtual TSharedRef<ITableRow> GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable) = 0;

	/** Generates a widget for the specified column name */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, int32 Depth, ITableRow* RowPtr) = 0;

	/** Gets an XML representation of the node's internals */
	virtual FString ToXML(const int32 indent = 0, bool bOutputKey = true) = 0;

	/** Refreshes anything necessary in the node, such as a text used in information displaying */
	virtual void Refresh() = 0;

	/** Calculate how many key/value pairs exist for this node. */
	virtual int32 GetNumPairs() = 0;

	/** Called when the filter changes */
	virtual void OnFilterTextChanged(FString NewFilter) = 0;

	/** When parents are refreshed, they can set the index of their children for proper displaying */
	virtual void SetIndex(int32 NewIndex) = 0;

public:

	/** Sets the KeyString of the node, needed for telling children to set their keys. Default do nothing. */
	virtual void SetKey(FString NewKey);
	/** Sets the value of the node. Default do nothing */
	virtual void SetValue(FString NewValue);
	/** Sets the value of the node. Default do nothing */
	virtual void SetValue(bool bNewValue);
	/** Sets a flag denoting whether the element is in an array. Default do nothing */
	virtual void SetArrayMember(bool bArrayMember);
	/** Gets the overlay brush dynamically */
	virtual const FSlateBrush* GetOverlayBrush();

public:

	/** Checks if a key string is valid */
	static bool IsKeyStringValid(FString ToCheck);
	/** Checks if a value string is valid */
	static bool IsValueStringValid(FString ToCheck);

public:

	/** Delegate: Gets the overlay brush from derived children classes */
	static const FSlateBrush* GetOverlayBrushDelegate(const TSharedRef<class IPListNode> In);

public:

	/** Sets the depth of the node */
	void SetDepth(int32 InDepth);
	/** Gets the depth of the node */
	int32 GetDepth();

protected:

	/** Pointer to the main editor widget for certain functionality, like marking it as dirty */
	SPListEditorPanel* EditorWidget;

private:

	int32 Depth;
};
