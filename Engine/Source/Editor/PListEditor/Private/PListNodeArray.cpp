// Copyright Epic Games, Inc. All Rights Reserved.

#include "PListNodeArray.h"

#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "PListNode.h"
#include "SPlistEditor.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"

class STableViewBase;
class SWidget;
struct FSlateBrush;

/** Validation check */
bool FPListNodeArray::IsValid()
{
	// Make sure all children are valid
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		if(!Children[i]->IsValid())
		{
			return false;
		}
	}

	// Check Key string
	if(KeyString.Len() == 0)
	{
		return false;
	}

	// No other cases. Array is valid
	return true;
}

/** Returns the array of children */
TArray<TSharedPtr<IPListNode> >& FPListNodeArray::GetChildren()
{
	return Children;
}

/** Adds a child to the internal array of the class */
void FPListNodeArray::AddChild(TSharedPtr<IPListNode> InChild)
{
	Children.Add(InChild);
}

/** Gets the type of the node via enum */
EPLNTypes FPListNodeArray::GetType()
{
	return EPLNTypes::PLN_Array;
}

/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
bool FPListNodeArray::UsesColumns()
{
	return true;
}

/** Generates a widget for a TableView row */
TSharedRef<ITableRow> FPListNodeArray::GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable)
{
	return GenerateInvalidRow(OwnerTable, NSLOCTEXT("PListNodeArray", "ArrayUsesColumns", "PListNodeArray uses columns") );
}

/** Generate a widget for the specified column name */
TSharedRef<SWidget> FPListNodeArray::GenerateWidgetForColumn(const FName& ColumnName, int32 InDepth, ITableRow* RowPtr)
{
	TableRow = RowPtr;
	check(TableRow);

	if(ColumnName == "PListKeyColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)

				// Space before expander
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
					.Size(FVector2D(20 * InDepth, 0))
				]

				// Editable key value
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(KeyStringTextBox, SEditableTextBox)
					.BackgroundColor(this, &FPListNodeArray::GetKeyBackgroundColor)
					.ForegroundColor(this, &FPListNodeArray::GetKeyForegroundColor)
					.Text(bArrayMember ? FText::FromString(FString::FromInt(ArrayIndex)) : FText::FromString(KeyString))
					.OnTextChanged(this, &FPListNodeArray::OnKeyStringChanged)
					.IsReadOnly(bArrayMember)
				]

				// Space before type column
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
					.Size(FVector2D(30, 0))
				]
			]

			// Expander button
			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				[
					// Space before expander
					SNew(SSpacer)
					.Size(FVector2D(20 * (InDepth - 1), 0))
				]

				+ SHorizontalBox::Slot()
				[
					SAssignNew(ExpanderArrow, SButton)
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					.ClickMethod( EButtonClickMethod::MouseDown )
					.Visibility( this, &FPListNodeArray::GetExpanderVisibility )
					.OnClicked( this, &FPListNodeArray::OnArrowClicked )
					.ContentPadding(2.1f)
					.ForegroundColor( FSlateColor::UseForeground() )
					[
						SNew(SImage)
						.Image( FAppStyle::GetBrush( TEXT("TreeArrow_Collapsed") ) )
						.Image( this, &FPListNodeArray::GetExpanderImage )
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			]
		];
	}

	else if(ColumnName == "PListValueTypeColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("PListEditor", "arrayValueTypeLabel", "array"))
		];
	}

	else if(ColumnName == "PListValueColumn")
	{
		return 
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SAssignNew(InfoTextWidget, STextBlock)
			.Text(FText::Format(NSLOCTEXT("PListEditor", "NumKeyValuePairsFmt", "[{0} key/value pairs]"), FText::AsNumber(GetNumPairs())))
		];
	}

	// Invalid column name
	else
	{
		return SNew(STextBlock) .Text(NSLOCTEXT("PListEditor", "UnknownColumn", "Unknown Column"));
	}
}

/** Gets an XML representation of the node's internals */
FString FPListNodeArray::ToXML(const int32 indent, bool bOutputKey)
{
	FString Output;

	// Output key
	Output += GenerateTabString(indent);
	Output += TEXT("<key>");
	Output += KeyString;
	Output += TEXT("</key>");
	Output += LINE_TERMINATOR;
	
	// Output array tag
	Output += GenerateTabString(indent);
	Output += TEXT("<array>");
	Output += LINE_TERMINATOR;

	// Output array contents
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Output += Children[i]->ToXML(indent + 1, false); // false: Child String/Bool shouldn't output key line
	}

	// Output array /tag
	Output += GenerateTabString(indent);
	Output += TEXT("</array>");
	Output += LINE_TERMINATOR;

	return Output;
}

/** Refreshes anything necessary in the node, such as a text used in information displaying */
void FPListNodeArray::Refresh()
{
	// Refresh display of # of child widgets (key/value pairs)
	if(InfoTextWidget.IsValid())
	{
		InfoTextWidget->SetText(FText::Format(NSLOCTEXT("PListEditor", "NumKeyValuePairsFmt", "[{0} key/value pairs]"), FText::AsNumber(GetNumPairs())));
	}

	// Refresh all children and set their indices
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Children[i]->SetIndex(i);
		Children[i]->Refresh();
	}

	// Refresh my own display of my index
	if(bArrayMember)
	{
		if(KeyStringTextBox.IsValid())
		{
			KeyStringTextBox->SetText(FText::FromString(FString::FromInt(ArrayIndex)));
		}
	}

	// Fix my box colors
	bKeyValid = IsKeyStringValid(KeyString);
}

/** Calculate how many key/value pairs exist for this node. */
int32 FPListNodeArray::GetNumPairs()
{
	int32 NumPairs = 0;
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		NumPairs += Children[i]->GetNumPairs();
	}
	return NumPairs;
}

/** Called when the filter changes */
void FPListNodeArray::OnFilterTextChanged(FString NewFilter)
{
	// Filter Key String
	if(NewFilter.Len() == 0)
	{
		bFiltered = false;
	}
	else
	{
		// Simple out case
		if(NewFilter.Len() > KeyString.Len() &&
			NewFilter.Len() > 5) // "array"
		{
			bFiltered = false;
		}
		else
		{
			FString FilterTextLower = NewFilter.ToLower();
			FString ArrayString = TEXT("array");
			if( ( KeyString.Len()	&& FCString::Strstr(KeyString.ToLower().GetCharArray().GetData(),	FilterTextLower.GetCharArray().GetData()) ) ||
				( ArrayString.Len() && FCString::Strstr(ArrayString.ToLower().GetCharArray().GetData(), FilterTextLower.GetCharArray().GetData()) ) )
			{
				bFiltered = true;
			}
			else
			{
				bFiltered = false;
			}
		}
	}

	// Pass on filter to children
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Children[i]->OnFilterTextChanged(NewFilter);
	}
}

/** When parents are refreshed, they can set the index of their children for proper displaying */
void FPListNodeArray::SetIndex(int32 NewIndex)
{
	check(NewIndex >= -1);
	ArrayIndex = NewIndex;
}

/** Sets the KeyString of the node, needed for telling children to set their keys */
void FPListNodeArray::SetKey(FString NewString)
{
	// Set new key
	KeyString = NewString;
	if(KeyStringTextBox.IsValid())
	{
		KeyStringTextBox->SetText(FText::FromString(NewString));
	}

	// Mark dirty
	check(EditorWidget);
	EditorWidget->MarkDirty();
}

/** Sets a flag denoting whether the element is in an array. Default do nothing */
void FPListNodeArray::SetArrayMember(bool bInArrayMember)
{
	bArrayMember = bInArrayMember;
}

/** Delegate: Gets the brush to use based on filter status */
const FSlateBrush* FPListNodeArray::GetOverlayBrush()
{
	if(bFiltered)
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.FilteredColor") );
	}
	else
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.NoOverlayColor") );
	}
}

/** Delegate: When key string changes */
void FPListNodeArray::OnKeyStringChanged(const FText& NewString)
{
	if(KeyString != NewString.ToString())
	{
		KeyString = NewString.ToString();
		bKeyValid = IsKeyStringValid(KeyString);

		check(EditorWidget);
		EditorWidget->MarkDirty();
	}
}

/** Delegate: Gets the image for the expander button */
const FSlateBrush* FPListNodeArray::GetExpanderImage() const
{
	check(TableRow != nullptr);

	const bool bIsItemExpanded = TableRow->IsItemExpanded();

	FName ResourceName;
	if (bIsItemExpanded)
	{
		if ( ExpanderArrow->IsHovered() )
		{
			ResourceName = "TreeArrow_Expanded_Hovered";
		}
		else
		{
			ResourceName = "TreeArrow_Expanded";
		}
	}
	else
	{
		if ( ExpanderArrow->IsHovered() )
		{
			ResourceName = "TreeArrow_Collapsed_Hovered";
		}
		else
		{
			ResourceName = "TreeArrow_Collapsed";
		}
	}

	return FAppStyle::GetBrush( ResourceName );
}

/** Delegate: Gets the visibility of the expander arrow */
EVisibility FPListNodeArray::GetExpanderVisibility() const
{
	if(Children.Num())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Hidden;
	}
}

/** Delegate: Handles when the arrow is clicked */
FReply FPListNodeArray::OnArrowClicked()
{
	check(TableRow);
	TableRow->ToggleExpansion();
	return FReply::Handled();
}

/** Delegate: Changes the color of the key string text box based on validity */
FSlateColor FPListNodeArray::GetKeyBackgroundColor() const
{
	if(bArrayMember)
	{
		return FLinearColor::White;
	}

	if(!bKeyValid)
	{
		return FAppStyle::GetColor("ErrorReporting.BackgroundColor");
	}
	else
	{
		return FLinearColor::White;
	}
}

/** Delegate: Changes the color of the key string text box based on validity */
FSlateColor FPListNodeArray::GetKeyForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");

	if(bArrayMember)
	{
		
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}

	if(!bKeyValid)
	{
		static const FName ForegroundColor("ErrorReporting.ForegroundColor");
		return FAppStyle::GetColor(ForegroundColor);
	}
	else
	{
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}
}
