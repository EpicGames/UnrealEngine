// Copyright Epic Games, Inc. All Rights Reserved.

#include "PListNodeBoolean.h"

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "PListNode.h"
#include "SPlistEditor.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

/** Validation check */
bool FPListNodeBoolean::IsValid()
{
	// Valid if key string is valid (and not an array member)
	if(bArrayMember)
	{
		return IsKeyStringValid(KeyString);
	}
	else
	{
		return true;
	}
}

/** Returns the array of children */
TArray<TSharedPtr<IPListNode> >& FPListNodeBoolean::GetChildren()
{
	// No internal array
	static TArray<TSharedPtr<IPListNode> > Empty;
	return Empty;
}

/** Adds a child to the internal array of the class */
void FPListNodeBoolean::AddChild(TSharedPtr<IPListNode> InChild)
{
	// Do nothing
}

/** Gets the type of the node via enum */
EPLNTypes FPListNodeBoolean::GetType()
{
	return EPLNTypes::PLN_Boolean;
}

/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
bool FPListNodeBoolean::UsesColumns()
{
	return true;
}

/** Generates a widget for a TableView row */
TSharedRef<ITableRow> FPListNodeBoolean::GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable)
{
	return GenerateInvalidRow(OwnerTable, NSLOCTEXT("PListNodeArray", "FPListNodeBooleanArrayUsesColumns", "FPListNodeBoolean uses columns"));
}

/** Generate a widget for the specified column name */
TSharedRef<SWidget> FPListNodeBoolean::GenerateWidgetForColumn(const FName& ColumnName, int32 InDepth, ITableRow* RowPtr)
{
	if(ColumnName == "PListKeyColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(SHorizontalBox)

			// Space before item representing expansion
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
				.Size(FVector2D(20 * InDepth, 0))
			]

			// Editable key value
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(KeyStringTextBox, SEditableTextBox)
				.BackgroundColor(this, &FPListNodeBoolean::GetKeyBackgroundColor)
				.ForegroundColor(this, &FPListNodeBoolean::GetKeyForegroundColor)
				.Text(bArrayMember ? FText::FromString(FString::FromInt(ArrayIndex)) : FText::FromString(KeyString))
				.OnTextChanged(this, &FPListNodeBoolean::OnKeyStringChanged)
				.IsReadOnly(bArrayMember)
			]
			
			// Space before type column
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
				.Size(FVector2D(30, 0))
			]
		];
	}

	else if(ColumnName == "PListValueTypeColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("PListEditor", "booleanValueTypeLabel", "boolean"))
		];
	}

	else if(ColumnName == "PListValueColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(SHorizontalBox)

			// Editable "value" value
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(ValueCheckBox, SCheckBox)
				.IsChecked(bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				.OnCheckStateChanged(this, &FPListNodeBoolean::OnValueChanged)
			]
		];
	}

	// Invalid column name
	else
	{
		return SNew(STextBlock) .Text(NSLOCTEXT("PListEditor", "UnknownColumn", "Unknown Column"));
	}
}

/** Gets an XML representation of the node's internals */
FString FPListNodeBoolean::ToXML(const int32 indent, bool bOutputKey)
{
	FString Output;

	// Output key line if necessary
	if(bOutputKey)
	{
		Output += GenerateTabString(indent);
		Output += TEXT("<key>");
		Output += KeyString;
		Output += TEXT("</key>");
		Output += LINE_TERMINATOR;
	}

	// Output value line
	if(bValue)
	{
		Output += GenerateTabString(indent);
		Output += TEXT("<true />");
		Output += LINE_TERMINATOR;
	}
	else
	{
		Output += GenerateTabString(indent);
		Output += TEXT("<false />");
		Output += LINE_TERMINATOR;
	}

	return Output;
}

/** Refreshes anything necessary in the node, such as a text used in information displaying */
void FPListNodeBoolean::Refresh()
{
	// Refresh my own display of my index
	if(bArrayMember)
	{
		if(KeyStringTextBox.IsValid())
		{
			KeyStringTextBox->SetText(FText::FromString(FString::FromInt(ArrayIndex)));
		}
	}

	// Fix my box colors
	bKeyValid = IsKeyStringValid(KeyString);
}

/** Calculate how many key/value pairs exist for this node. */
int32 FPListNodeBoolean::GetNumPairs()
{
	return 1;
}

/** Called when the filter changes */
void FPListNodeBoolean::OnFilterTextChanged(FString NewFilter)
{
	// Filter Key String
	if(NewFilter.Len() == 0)
	{
		bFiltered = false;
	}
	else
	{
		// Simple out case
		if(NewFilter.Len() > KeyString.Len() &&
			NewFilter.Len() > 7) // > "boolean"
		{
			bFiltered = false;
		}
		else
		{
			FString FilterTextLower = NewFilter.ToLower();
			FString ValueString = bValue ? TEXT("true") : TEXT("false");
			FString BooleanString = TEXT("boolean");
			if( (KeyString.Len()		&& FCString::Strstr(KeyString.ToLower().GetCharArray().GetData(),		FilterTextLower.GetCharArray().GetData()) ) ||
				(BooleanString.Len()	&& FCString::Strstr(BooleanString.ToLower().GetCharArray().GetData(),	FilterTextLower.GetCharArray().GetData()) ) ||
				(ValueString.Len()		&& FCString::Strstr(ValueString.ToLower().GetCharArray().GetData(),		FilterTextLower.GetCharArray().GetData()) ) )
			{
				bFiltered = true;
			}
			else
			{
				bFiltered = false;
			}
		}
	}
}

/** When parents are refreshed, they can set the index of their children for proper displaying */
void FPListNodeBoolean::SetIndex(int32 NewIndex)
{
	check(NewIndex >= -1);
	ArrayIndex = NewIndex;
}

/** Sets the KeyString of the node, needed for telling children to set their keys */
void FPListNodeBoolean::SetKey(FString NewString)
{
	// Set new key
	KeyString = NewString;
	if(KeyStringTextBox.IsValid())
	{
		KeyStringTextBox->SetText(FText::FromString(NewString));
	}

	// Mark dirty
	check(EditorWidget);
	EditorWidget->MarkDirty();
}

/** Sets the value of the boolean */
void FPListNodeBoolean::SetValue(bool bNewValue)
{
	// Set new value
	if(bNewValue != bValue)
	{
		bValue = bNewValue;
		if(ValueCheckBox.IsValid())
		{
			ValueCheckBox->ToggleCheckedState();
		}

		// Mark dirty
		check(EditorWidget);
		EditorWidget->MarkDirty();
	}
}

/** Sets a flag denoting whether the element is in an array */
void FPListNodeBoolean::SetArrayMember(bool bNewArrayMember)
{
	// Changing this after the widget has been generated won't change the displayed widgets..
	// So be sure to call function on initialization.
	bArrayMember = bNewArrayMember;
}

/** Gets the brush to use based on filter status */
const FSlateBrush* FPListNodeBoolean::GetOverlayBrush()
{
	if(bFiltered)
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.FilteredColor") );
	}
	else
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.NoOverlayColor") );
	}
}

/** Delegate: When key string changes */
void FPListNodeBoolean::OnKeyStringChanged(const FText& NewString)
{
	if(KeyString != NewString.ToString())
	{
		KeyString = NewString.ToString();

		check(KeyStringTextBox.IsValid());
		bKeyValid = IsKeyStringValid(KeyString);

		check(EditorWidget);
		EditorWidget->MarkDirty();
	}
}

/** Delegate: When the checkbox changes */
void FPListNodeBoolean::OnValueChanged(ECheckBoxState NewValue)
{
	if(NewValue == ECheckBoxState::Checked)
	{
		// Simply do nothing
		if(bValue == true)
		{
			return;
		}

		bValue = true;

		// Dirty flag
		check(EditorWidget);
		EditorWidget->MarkDirty();
	}
	else if(NewValue == ECheckBoxState::Unchecked)
	{
		// Simply do nothing
		if(bValue == false)
		{
			return;
		}

		bValue = false;

		// Dirty flag
		check(EditorWidget);
		EditorWidget->MarkDirty();
	}
	else
	{
		check(!TEXT("Checkbox was uninitialized and has an undetermined value!"));
	}
}

/** Delegate: Changes the color of the key string text box based on validity */
FSlateColor FPListNodeBoolean::GetKeyBackgroundColor() const
{
	if(bArrayMember)
	{
		return FLinearColor::White;
	}

	if(!bKeyValid)
	{
		return FAppStyle::GetColor("ErrorReporting.BackgroundColor");
	}
	else
	{
		return FLinearColor::White;
	}
}

/** Delegate: Changes the color of the key string text box based on validity */
FSlateColor FPListNodeBoolean::GetKeyForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");

	if(bArrayMember)
	{
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}

	if(!bKeyValid)
	{
		static const FName ForegroundColor("ErrorReporting.ForegroundColor");
		return FAppStyle::GetColor(ForegroundColor);
	}
	else
	{
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}
}
