// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "PListNode.h"
#include "HAL/Platform.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FText;
class ITableRow;
class SCheckBox;
class SEditableTextBox;
class SPListEditorPanel;
class STableViewBase;
class SWidget;
struct FSlateBrush;

enum class ECheckBoxState : uint8;

/** A Node representing a boolean value */
class FPListNodeBoolean : public IPListNode
{
public:
	FPListNodeBoolean(SPListEditorPanel* InEditorWidget)
		: IPListNode(InEditorWidget), bValue(false), ArrayIndex(-1), bFiltered(false), bArrayMember(false), bKeyValid(false)
	{}

public:

	/** Validation check */
	virtual bool IsValid() override;

	/** Returns the array of children */
	virtual TArray<TSharedPtr<IPListNode> >& GetChildren() override;

	/** Adds a child to the internal array of the class */
	virtual void AddChild(TSharedPtr<IPListNode> InChild) override;

	/** Gets the type of the node via enum */
	virtual EPLNTypes GetType() override;

	/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
	virtual bool UsesColumns() override;

	/** Generates a widget for a TableView row */
	virtual TSharedRef<ITableRow> GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable) override;

	/** Generate a widget for the specified column name */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, int32 Depth, ITableRow* RowPtr) override;

	/** Gets an XML representation of the node's internals */
	virtual FString ToXML(const int32 indent = 0, bool bOutputKey = true) override;

	/** Refreshes anything necessary in the node, such as a text used in information displaying */
	virtual void Refresh() override;

	/** Calculate how many key/value pairs exist for this node. */
	virtual int32 GetNumPairs() override;

	/** Called when the filter changes */
	virtual void OnFilterTextChanged(FString NewFilter) override;

	/** When parents are refreshed, they can set the index of their children for proper displaying */
	virtual void SetIndex(int32 NewIndex) override;

public:
	
	/** Gets the brush to use based on filter status */
	const FSlateBrush* GetOverlayBrush() override;
	/** Sets the KeyString of the node, needed for telling children to set their keys */
	virtual void SetKey(FString NewString) override;
	/** Sets the value of the boolean */
	virtual void SetValue(bool bNewValue) override;
	/** Sets a flag denoting whether the element is in an array */
	virtual void SetArrayMember(bool bArrayMember) override;

private:

	/** Delegate: When key string changes */
	void OnKeyStringChanged(const FText& NewString);
	/** Delegate: When the checkbox changes */
	void OnValueChanged(ECheckBoxState NewValue);
	/** Delegate: Changes the color of the key string text box based on validity */
	FSlateColor GetKeyBackgroundColor() const;
	/** Delegate: Changes the color of the key string text box based on validity */
	FSlateColor GetKeyForegroundColor() const;

private:

	/** The string for the key */
	FString KeyString;
	/** The editable text box for the key string */
	TSharedPtr<SEditableTextBox> KeyStringTextBox;

	/** The value */
	bool bValue;
	/** The editable text box for the value string */
	TSharedPtr<SCheckBox> ValueCheckBox;

	/** Index within parent array. Ignore if not an array member */
	int32 ArrayIndex;
	/** Filtered flag */
	bool bFiltered;
	/** Array flag */
	bool bArrayMember;

	/** Flag for valid key string */
	bool bKeyValid;
};
