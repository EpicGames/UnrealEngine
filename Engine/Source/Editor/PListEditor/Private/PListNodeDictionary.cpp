// Copyright Epic Games, Inc. All Rights Reserved.

#include "PListNodeDictionary.h"

#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "PListNode.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"

/** Validation check */
bool FPListNodeDictionary::IsValid()
{
	// Dictionary is valid if all its children are valid
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		if(!Children[i]->IsValid())
		{
			return false;
		}
	}

	// No other cases. Valid.
	return true;
}

/** Returns the array of children */
TArray<TSharedPtr<IPListNode> >& FPListNodeDictionary::GetChildren()
{
	return Children;
}

/** Adds a child to the internal array of the class */
void FPListNodeDictionary::AddChild(TSharedPtr<IPListNode> InChild)
{
	Children.Add(InChild);
}

/** Gets the type of the node via enum */
EPLNTypes FPListNodeDictionary::GetType()
{
	return EPLNTypes::PLN_Dictionary;
}

/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
bool FPListNodeDictionary::UsesColumns()
{
	return true;
}

/** Generates a widget for a TableView row */
TSharedRef<ITableRow> FPListNodeDictionary::GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable)
{
	return GenerateInvalidRow(OwnerTable, NSLOCTEXT("PListNodeArray", "FPListNodeDictionaryArrayUsesColumns", "FPListNodeDictionary uses columns"));
}

/** Generate a widget for the specified column name */
TSharedRef<SWidget> FPListNodeDictionary::GenerateWidgetForColumn(const FName& ColumnName, int32 InDepth, ITableRow* RowPtr)
{
	TableRow = RowPtr;
	check(TableRow);

	if(ColumnName == "PListKeyColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
		
				// Space item representing item expansion
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
					.Size(FVector2D(20 * InDepth, 0))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(KeyStringTextBox, SEditableTextBox)
					.Text(bArrayMember ? FText::FromString(FString::FromInt(ArrayIndex)) : FText::FromString(TEXT("dictionary")))
					.IsReadOnly(true)
				]

				// Spacer between type
				+ SHorizontalBox::Slot()
				[
					SNew(SSpacer)
					.Size(FVector2D(30, 0))
				]
			]
		
			// Expander for innards
			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				[
					// Space before expander
					SNew(SSpacer)
					.Size(FVector2D(20 * (InDepth - 1), 0))
				]

				+ SHorizontalBox::Slot()
				[
					SAssignNew(ExpanderArrow, SButton)
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					.ClickMethod( EButtonClickMethod::MouseDown )
					.Visibility( this, &FPListNodeDictionary::GetExpanderVisibility )
					.OnClicked( this, &FPListNodeDictionary::OnArrowClicked )
					.ContentPadding(2.1f)
					.ForegroundColor( FSlateColor::UseForeground() )
					[
						SNew(SImage)
						.Image( this, &FPListNodeDictionary::GetExpanderImage )
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			]
		];
	}

	else if(ColumnName == "PListValueTypeColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("PListEditor", "dictionaryValueTypeLabel", "dictionary"))
		];
	}

	else if(ColumnName == "PListValueColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SAssignNew(InfoTextWidget, STextBlock)
			.Text(FText::Format(NSLOCTEXT("PListEditor", "NumKeyValuePairsFmt", "[{0} key/value pairs]"), FText::AsNumber(GetNumPairs())))
		];
	}

	// Invalid column name
	else
	{
		return SNew(STextBlock) .Text(NSLOCTEXT("PListEditor", "UnknownColumn", "Unknown Column"));
	}
}

/** Gets an XML representation of the node's internals */
FString FPListNodeDictionary::ToXML(const int32 indent, bool bOutputKey)
{
	FString Output;

	// Dictionary header
	Output += GenerateTabString(indent);
	Output += TEXT("<dict>");
	Output += LINE_TERMINATOR;

	// Dictionary contents
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Output += Children[i]->ToXML(indent + 1);
	}

	// Dictionary footer
	Output += GenerateTabString(indent);
	Output += TEXT("</dict>");
	Output += LINE_TERMINATOR;

	return Output;
}

/** Refreshes anything necessary in the node, such as a text used in information displaying */
void FPListNodeDictionary::Refresh()
{
	// Update the display of the number of key/value pairs
	if(InfoTextWidget.IsValid())
	{
		InfoTextWidget->SetText(FText::Format(NSLOCTEXT("PListEditor", "NumKeyValuePairsFmt", "[{0} key/value pairs]"), FText::AsNumber(GetNumPairs())));
	}

	// Refresh all children
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Children[i]->Refresh();
	}

	// Refresh my own display of my index
	if(bArrayMember)
	{
		if(KeyStringTextBox.IsValid())
		{
			KeyStringTextBox->SetText(FText::FromString(FString::FromInt(ArrayIndex)));
		}
	}
}

/** Calculate how many key/value pairs exist for this node. */
int32 FPListNodeDictionary::GetNumPairs()
{
	int32 NumPairs = 0;
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		NumPairs += Children[i]->GetNumPairs();
	}
	return NumPairs;
}

/** Called when the filter changes */
void FPListNodeDictionary::OnFilterTextChanged(FString NewFilter)
{
	// Empty filter
	if(NewFilter.Len() == 0)
	{
		bFiltered = false;
	}
	else
	{
		// Simple out case
		if(NewFilter.Len() > 10) // "dictionary"
		{
			bFiltered = false;
		}
		else
		{
			FString FilterTextLower = NewFilter.ToLower();
			FString DictString = TEXT("dictionary");
			if( FCString::Strstr(DictString.ToLower().GetCharArray().GetData(),	FilterTextLower.GetCharArray().GetData())  )
			{
				bFiltered = true;
			}
			else
			{
				bFiltered = false;
			}
		}
	}

	// Pass filter on to children
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Children[i]->OnFilterTextChanged(NewFilter);
	}
}

/** When parents are refreshed, they can set the index of their children for proper displaying */
void FPListNodeDictionary::SetIndex(int32 NewIndex)
{
	check(NewIndex >= -1);
	ArrayIndex = NewIndex;
}

/** Sets a flag denoting whether the element is in an array. Default do nothing */
void FPListNodeDictionary::SetArrayMember(bool bInArrayMember)
{
	bArrayMember = bInArrayMember;
}

/** Gets the overlay brush dynamically */
const FSlateBrush* FPListNodeDictionary::GetOverlayBrush()
{
	if(bFiltered)
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.FilteredColor") );
	}
	else
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.NoOverlayColor") );
	}
}

/** Delegate: Gets the image for the expander button */
const FSlateBrush* FPListNodeDictionary::GetExpanderImage() const
{
	check(TableRow != nullptr);

	const bool bIsItemExpanded = TableRow->IsItemExpanded();

	FName ResourceName;
	if (bIsItemExpanded)
	{
		if ( ExpanderArrow->IsHovered() )
		{
			ResourceName = "TreeArrow_Expanded_Hovered";
		}
		else
		{
			ResourceName = "TreeArrow_Expanded";
		}
	}
	else
	{
		if ( ExpanderArrow->IsHovered() )
		{
			ResourceName = "TreeArrow_Collapsed_Hovered";
		}
		else
		{
			ResourceName = "TreeArrow_Collapsed";
		}
	}

	return FAppStyle::GetBrush( ResourceName );
}

/** Delegate: Gets the visibility of the expander arrow */
EVisibility FPListNodeDictionary::GetExpanderVisibility() const
{
	if(Children.Num())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Hidden;
	}
}

/** Delegate: Handles when the arrow is clicked */
FReply FPListNodeDictionary::OnArrowClicked()
{
	check(TableRow);
	TableRow->ToggleExpansion();
	return FReply::Handled();
}
