// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "PListNode.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ITableRow;
class SButton;
class SEditableTextBox;
class SPListEditorPanel;
class STableViewBase;
class STextBlock;
class SWidget;
struct FSlateBrush;

/** A Node representing a dictionary */
class FPListNodeDictionary : public IPListNode
{
public:
	FPListNodeDictionary(SPListEditorPanel* InEditorWidget)
		: IPListNode(InEditorWidget), TableRow(nullptr), ArrayIndex(-1), bArrayMember(false), bFiltered(false)
	{}

public:

	/** Validation check */
	virtual bool IsValid() override;

	/** Returns the array of children */
	virtual TArray<TSharedPtr<IPListNode> >& GetChildren() override;

	/** Adds a child to the internal array of the class */
	virtual void AddChild(TSharedPtr<IPListNode> InChild) override;

	/** Gets the type of the node via enum */
	virtual EPLNTypes GetType() override;

	/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
	virtual bool UsesColumns() override;

	/** Generates a widget for a TableView row */
	virtual TSharedRef<ITableRow> GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable) override;

	/** Generate a widget for the specified column name */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, int32 Depth, ITableRow* RowPtr) override;

	/** Gets an XML representation of the node's internals */
	virtual FString ToXML(const int32 indent = 0, bool bOutputKey = true) override;

	/** Refreshes anything necessary in the node, such as a text used in information displaying */
	virtual void Refresh() override;

	/** Calculate how many key/value pairs exist for this node. */
	virtual int32 GetNumPairs() override;

	/** Called when the filter changes */
	virtual void OnFilterTextChanged(FString NewFilter) override;

	/** When parents are refreshed, they can set the index of their children for proper displaying */
	virtual void SetIndex(int32 NewIndex) override;

public:

	/** Sets a flag denoting whether the element is in an array. Default do nothing */
	virtual void SetArrayMember(bool bArrayMember) override;
	/** Gets the overlay brush dynamically */
	virtual const FSlateBrush* GetOverlayBrush() override;

private:

	/** Delegate: Gets the image for the expander button */
	const FSlateBrush* GetExpanderImage() const;
	/** Delegate: Gets the visibility of the expander arrow */
	EVisibility GetExpanderVisibility() const;
	/** Delegate: Handles when the arrow is clicked */
	FReply OnArrowClicked();

private:

	/** All children of the dictionary */
	TArray<TSharedPtr<IPListNode> > Children;
	
	/** The editable text box for the key string */
	TSharedPtr<SEditableTextBox> KeyStringTextBox;
	/** Info text widget for displaying num children */
	TSharedPtr<STextBlock> InfoTextWidget;

	/** Widget for the expander arrow */
	TSharedPtr<SButton> ExpanderArrow;
	/** Reference to the Containing Row */
	ITableRow* TableRow;

	/** Index within parent array. Ignore if not an array member */
	int32 ArrayIndex;
	/** Array Member Status */
	bool bArrayMember;
	/** Filtered Status */
	bool bFiltered;
};
