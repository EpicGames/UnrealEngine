// Copyright Epic Games, Inc. All Rights Reserved.

#include "PListNodeFile.h"

#include "Containers/BitArray.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "PListNode.h"
#include "SlotBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"


/** Validation check */
bool FPListNodeFile::IsValid()
{
	// File is valid if all of its contents are
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		if(!Children[i]->IsValid())
		{
			return false;
		}
	}
	return true;
}

/** Returns the array of children */
TArray<TSharedPtr<IPListNode> >& FPListNodeFile::GetChildren()
{
	return Children;
}

/** Adds a child to the internal array of the class */
void FPListNodeFile::AddChild(TSharedPtr<IPListNode> InChild) 
{
	Children.Add(InChild);
}

/** Gets the type of the node via enum */
EPLNTypes FPListNodeFile::GetType() 
{
	return EPLNTypes::PLN_File;
}

/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
bool FPListNodeFile::UsesColumns()
{
	return false;
}

/** Generates a widget for a TableView row */
TSharedRef<ITableRow> FPListNodeFile::GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable)
{
	return
	SNew(STableRow<TSharedPtr<ITableRow> >, OwnerTable)
	.Content()
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 1)
		[
			SAssignNew(TextWidget, STextBlock)
			.Text(FText::Format(NSLOCTEXT("PListEditor", "FileAndNumKeyValuePairsFmt", "file [{0} key/value pairs]"), FText::AsNumber(GetNumPairs())))
		]
	];
}

/** Generate a widget for the specified column name */
TSharedRef<SWidget> FPListNodeFile::GenerateWidgetForColumn(const FName& ColumnName, int32 InDepth, ITableRow* RowPtr) 
{
	return GenerateInvalidRow(NSLOCTEXT("PListNodeArray", "PListNodeFileArrayUsesColumns", "PListNodeFile does not use columns"));
}

/** Gets an XML representation of the node's internals */
FString FPListNodeFile::ToXML(const int32 indent, bool /*bOutputKey*/) 
{
	// PList Header
	FString Output;
	(Output += GenerateTabString(indent) + TEXT("<?xml version=\"1.0\" encoding=\"UTF-8\"?>")) += LINE_TERMINATOR;
	(Output += GenerateTabString(indent) + TEXT("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">")) += LINE_TERMINATOR;
	(Output += GenerateTabString(indent) + TEXT("<plist version=\"1.0\">")) += LINE_TERMINATOR;

	// Get XML contents of children
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Output += Children[i]->ToXML(indent + 1);
	}

	// PList Footer
	(Output += GenerateTabString(indent) + TEXT("</plist>")) /*+= LINE_TERMINATOR*/; // Last line without newline

	return Output;
}

/** Refreshes anything necessary in the node, such as a text used in information displaying */
void FPListNodeFile::Refresh()
{
	// Refresh internals
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Children[i]->Refresh();
	}

	// Calculate how many contained key/value pairs exist
	if(TextWidget.IsValid())
	{
		TextWidget->SetText(FText::Format(NSLOCTEXT("PListEditor", "FileAndNumKeyValuePairsFmt", "file [{0} key/value pairs]"), FText::AsNumber(GetNumPairs())));
	}
}

/** Calculate how many key/value pairs exist for this node. */
int32 FPListNodeFile::GetNumPairs()
{
	int32 Total = 0;
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Total += Children[i]->GetNumPairs();
	}
	return Total;
}

/** Called when the filter changes */
void FPListNodeFile::OnFilterTextChanged(FString NewFilter)
{
	// Let all children know filter has changed
	for(int32 i = 0; i < Children.Num(); ++i)
	{
		Children[i]->OnFilterTextChanged(NewFilter);
	}
}

/** When parents are refreshed, they can set the index of their children for proper displaying */
void FPListNodeFile::SetIndex(int32 /*NewIndex*/)
{
	// Do nothing
}
