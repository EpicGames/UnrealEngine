// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "PListNode.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ITableRow;
class SPListEditorPanel;
class STableViewBase;
class STextBlock;
class SWidget;

/** A Node representing the plist as a whole */
class FPListNodeFile : public IPListNode
{
public:
	FPListNodeFile(SPListEditorPanel* InEditorWidget)
		: IPListNode(InEditorWidget)
	{}

public:

	/** Validation check */
	virtual bool IsValid() override;

	/** Returns the array of children */
	virtual TArray<TSharedPtr<IPListNode> >& GetChildren() override;

	/** Adds a child to the internal array of the class */
	virtual void AddChild(TSharedPtr<IPListNode> InChild) override;

	/** Gets the type of the node via enum */
	virtual EPLNTypes GetType() override;

	/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
	virtual bool UsesColumns() override;

	/** Generates a widget for a TableView row */
	virtual TSharedRef<ITableRow> GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable) override;

	/** Generate a widget for the specified column name */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, int32 Depth, ITableRow* RowPtr) override;

	/** Gets an XML representation of the node's internals */
	virtual FString ToXML(const int32 indent = 0, bool bOutputKey = true) override;

	/** Refreshes anything necessary in the node, such as a text used in information displaying */
	virtual void Refresh() override;

	/** Calculate how many key/value pairs exist for this node. */
	virtual int32 GetNumPairs() override;

	/** Called when the filter changes */
	virtual void OnFilterTextChanged(FString NewFilter) override;

	/** When parents are refreshed, they can set the index of their children for proper displaying */
	virtual void SetIndex(int32 NewIndex) override;

private:

	/** All children of the file (everything) */
	TArray<TSharedPtr<IPListNode> > Children;
	/** Widget for displaying text on this row */
	TSharedPtr<STextBlock> TextWidget;
};
