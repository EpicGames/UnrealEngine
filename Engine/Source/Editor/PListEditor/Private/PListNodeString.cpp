// Copyright Epic Games, Inc. All Rights Reserved.

#include "PListNodeString.h"

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "PListNode.h"
#include "SPlistEditor.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

/** Validation check */
bool FPListNodeString::IsValid()
{
	// Check Key (if we're not an array member)
	if(bArrayMember)
	{
		if(KeyString.Len() == 0)
		{
			return false;
		}
	}

	// Check Value
	if(ValueString.Len() == 0)
	{
		return false;
	}

	// No other cases, so valid
	return true;
}

/** Returns the array of children */
TArray<TSharedPtr<IPListNode> >& FPListNodeString::GetChildren()
{
	// No internal children
	static TArray<TSharedPtr<IPListNode> > Empty;
	return Empty;
}

/** Adds a child to the internal array of the class */
void FPListNodeString::AddChild(TSharedPtr<IPListNode> InChild)
{
	// Do nothing
}

/** Gets the type of the node via enum */
EPLNTypes FPListNodeString::GetType()
{
	return EPLNTypes::PLN_String;
}

/** Determines whether the node needs to generate widgets for columns, or just use the whole row without columns */
bool FPListNodeString::UsesColumns()
{
	return true;
}

/** Generates a widget for a TableView row */
TSharedRef<ITableRow> FPListNodeString::GenerateWidget(const TSharedRef<STableViewBase>& OwnerTable)
{
	return GenerateInvalidRow(OwnerTable, NSLOCTEXT("FPListNodeString", "ArrayUsesColumns", "FPListNodeString uses columns"));
}

/** Generate a widget for the specified column name */
TSharedRef<SWidget> FPListNodeString::GenerateWidgetForColumn(const FName& ColumnName, int32 InDepth, ITableRow* RowPtr)
{
	if(ColumnName == "PListKeyColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(SHorizontalBox)

			// Space item representing item expansion
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
				.Size(FVector2D(20 * InDepth, 0))
			]

			// Editable key value
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(KeyStringTextBox, SEditableTextBox)
				.BackgroundColor(this, &FPListNodeString::GetKeyBackgroundColor)
				.ForegroundColor(this, &FPListNodeString::GetKeyForegroundColor)
				.Text(bArrayMember ? FText::FromString(FString::FromInt(ArrayIndex)) : FText::FromString(KeyString))
				.OnTextChanged(this, &FPListNodeString::OnKeyStringChanged)
				.IsReadOnly(bArrayMember)
			]

			// Spacer between type
			+ SHorizontalBox::Slot()
			[
				SNew(SSpacer)
				.Size(FVector2D(30, 0))
			]
		];
	}

	else if(ColumnName == "PListValueTypeColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("PListEditor", "stringValueTypeLabel", "string"))
		];
	}

	else if(ColumnName == "PListValueColumn")
	{
		return
		SNew(SBorder)
		.BorderImage_Static(&IPListNode::GetOverlayBrushDelegate, AsShared())
		[
			SNew(SHorizontalBox)

			// Editable "value" value
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(ValueStringTextBox, SEditableTextBox)
				.BackgroundColor(this, &FPListNodeString::GetValueBackgroundColor)
				.ForegroundColor(this, &FPListNodeString::GetValueForegroundColor)
				.Text(FText::FromString(ValueString))
				.OnTextChanged(this, &FPListNodeString::OnValueStringChanged)
			]
		];
	}

	// Invalid column name
	else
	{
		return SNew(STextBlock) .Text(NSLOCTEXT("PListEditor", "UnknownColumn", "Unknown Column"));
	}
}

/** Gets an XML representation of the node's internals */
FString FPListNodeString::ToXML(const int32 indent, bool bOutputKey)
{
	FString Output;

	// Output key line if necessary
	if(bOutputKey)
	{
		Output += GenerateTabString(indent);
		Output += TEXT("<key>");
		Output += KeyString;
		Output += TEXT("</key>");
		Output += LINE_TERMINATOR;
	}

	// Output value line
	Output += GenerateTabString(indent);
	Output += TEXT("<string>");
	Output += ValueString;
	Output += TEXT("</string>");
	Output += LINE_TERMINATOR;

	return Output;
}

/** Refreshes anything necessary in the node, such as a text used in information displaying */
void FPListNodeString::Refresh()
{
	// Refresh my own display of my index
	if(bArrayMember)
	{
		if(KeyStringTextBox.IsValid())
		{
			KeyStringTextBox->SetText(FText::FromString(FString::FromInt(ArrayIndex)));
		}
	}

	// Fix my box colors
	bKeyValid = IsKeyStringValid(KeyString);
	bValueValid = IsValueStringValid(ValueString);
}

/** Calculate how many key/value pairs exist for this node. */
int32 FPListNodeString::GetNumPairs()
{
	return 1;
}

/** Called when the filter changes */
void FPListNodeString::OnFilterTextChanged(FString NewFilter)
{
	// Filter Key String
	if(NewFilter.Len() == 0)
	{
		bFiltered = false;
	}
	else
	{
		// Simple out case
		if(NewFilter.Len() > KeyString.Len() &&
			NewFilter.Len() > ValueString.Len() &&
			NewFilter.Len() > 6) // "string"
		{
			bFiltered = false;
		}
		else
		{
			FString FilterTextLower = NewFilter.ToLower();
			FString StringText = TEXT("string");
			if( (KeyString.Len()	&& FCString::Strstr(KeyString.ToLower().GetCharArray().GetData(),	FilterTextLower.GetCharArray().GetData()) ) ||
				(StringText.Len()	&& FCString::Strstr(StringText.ToLower().GetCharArray().GetData(),	FilterTextLower.GetCharArray().GetData()) ) ||
				(ValueString.Len()	&& FCString::Strstr(ValueString.ToLower().GetCharArray().GetData(),	FilterTextLower.GetCharArray().GetData()) ) )
			{
				bFiltered = true;
			}
			else
			{
				bFiltered = false;
			}
		}
	}
}

/** When parents are refreshed, they can set the index of their children for proper displaying */
void FPListNodeString::SetIndex(int32 NewIndex)
{
	check(NewIndex >= -1);
	ArrayIndex = NewIndex;
}

/** Sets the KeyString of the node, needed for telling children to set their keys */
void FPListNodeString::SetKey(FString NewString)
{
	// Set new key
	KeyString = NewString;
	if(KeyStringTextBox.IsValid())
	{
		KeyStringTextBox->SetText(FText::FromString(NewString));
	}

	// Mark dirty
	check(EditorWidget);
	EditorWidget->MarkDirty();
}

/** Sets the value of the boolean */
void FPListNodeString::SetValue(FString NewValue)
{
	// Set new value
	ValueString = NewValue;
	if(ValueStringTextBox.IsValid())
	{
		ValueStringTextBox->SetText(FText::FromString(ValueString));
	}

	// Mark dirty
	check(EditorWidget);
	EditorWidget->MarkDirty();
}

/** Sets a flag denoting whether the element is in an array */
void FPListNodeString::SetArrayMember(bool bNewArrayMember)
{
	// Changing this after the widget has been generated won't change the displayed widgets..
	// So be sure to call function on initialization.
	bArrayMember = bNewArrayMember;
}

/** Gets the brush to use based on filter status */
const FSlateBrush* FPListNodeString::GetOverlayBrush()
{
	if(bFiltered)
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.FilteredColor") );
	}
	else
	{
		return FAppStyle::GetBrush( TEXT("PListEditor.NoOverlayColor") );
	}
}

/** Delegate: When key string changes */
void FPListNodeString::OnKeyStringChanged(const FText& NewString)
{
	if(KeyString != NewString.ToString())
	{
		KeyString = NewString.ToString();

		check(KeyStringTextBox.IsValid());
		bKeyValid = IsKeyStringValid(KeyString);

		check(EditorWidget);
		EditorWidget->MarkDirty();
	}
}

/** Delegate: When the value changes */
void FPListNodeString::OnValueStringChanged(const FText& NewString)
{
	if(ValueString != NewString.ToString())
	{
		ValueString = NewString.ToString();

		check(ValueStringTextBox.IsValid());
		bValueValid = IsValueStringValid(ValueString);

		check(EditorWidget);
		EditorWidget->MarkDirty();
	}
}

/** Delegate: Changes the color of the key string text box based on validity */
FSlateColor FPListNodeString::GetKeyBackgroundColor() const
{
	if(bArrayMember)
	{
		return FLinearColor::White;
	}

	if(!bKeyValid)
	{
		return FAppStyle::GetColor("ErrorReporting.BackgroundColor");
	}
	else
	{
		return FLinearColor::White;
	}
}

/** Delegate: Changes the color of the key string text box based on validity */
FSlateColor FPListNodeString::GetKeyForegroundColor() const
{
	static const FName InvertedForegroundName("InvertedForeground");

	if(bArrayMember)
	{
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}

	if(!bKeyValid)
	{
		static const FName ForegroundColor("ErrorReporting.ForegroundColor");
		return FAppStyle::GetColor(ForegroundColor);
	}
	else
	{
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}
}


/** Delegate: Changes the color of the value string text box based on validity */
FSlateColor FPListNodeString::GetValueBackgroundColor() const
{
	if(!bValueValid)
	{
		static const FName BackgroundColor("ErrorReporting.BackgroundColor");
		return FAppStyle::GetColor(BackgroundColor);
	}
	else
	{
		return FLinearColor::White;
	}
}

/** Delegate: Changes the color of the value string text box based on validity */
FSlateColor FPListNodeString::GetValueForegroundColor() const
{
	if(!bValueValid)
	{
		static const FName ForegroundColor("ErrorReporting.ForegroundColor");
		return FAppStyle::GetSlateColor(ForegroundColor);
	}
	else
	{
		static const FName InvertedForegroundName("InvertedForeground");
		return FAppStyle::GetSlateColor(InvertedForegroundName);
	}
}

