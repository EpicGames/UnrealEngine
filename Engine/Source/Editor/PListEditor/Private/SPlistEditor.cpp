// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlistEditor.h"

#include "Containers/StringConv.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMisc.h"
#include "IDesktopPlatform.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "PListNode.h"
#include "PListNodeArray.h"
#include "PListNodeBoolean.h"
#include "PListNodeDictionary.h"
#include "PListNodeFile.h"
#include "PListNodeString.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "XmlFile.h"
#include "XmlNode.h"

class ITableRow;
class SWidget;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "PListEditor"

/** Constructs the main widget for the editor */
void SPListEditorPanel::Construct( const FArguments& InArgs )
{
	// Set defaults for the editor
	FString LoadedFileName = LOCTEXT("PListNoFileLoaded", "No File Loaded").ToString();
	bFileLoaded = false;
	InOutLastPath = FPaths::ProjectDir() + TEXT("Build/IOS/");
	bPromptSave = true;
	bNewFile = false;
	bDirty = false;
	bPromptDelete = true;
	FramesToSkip = 0;

	ChildSlot
	.Padding(1)
	[
		SNew(SOverlay)
		
		// Main Content
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.Content()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush("PListEditor.HeaderRow.Background") )
					.Content()
					[
						SNew(SVerticalBox)

						// Add/New/etc buttons
						+SVerticalBox::Slot()
						.Padding(2)
						[
							SNew(SHorizontalBox)

							// File Menu
							+SHorizontalBox::Slot()
							.Padding(0, 0, 2, 0)
							.AutoWidth()
							[
								SNew(SExpandableArea)
								.InitiallyCollapsed(true)
								.AreaTitle(LOCTEXT("PListMenuTitle", "File"))
								.BodyContent()
								[
									SNew(SHorizontalBox)

									// New Button
									+SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(2, 0)
									.HAlign(HAlign_Left)
									[
										SNew(SButton)
										.Text(LOCTEXT("PListNew", "New"))
										.ToolTipText(LOCTEXT("PListNewToolTip", "Create a new plist file"))
										.OnClicked(this, &SPListEditorPanel::OnNewClicked)
									]

									// Open Button
									+SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(2, 0)
									.HAlign(HAlign_Left)
									[
										SNew(SButton)
										.Text(LOCTEXT("PListOpen", "Open..."))
										.ToolTipText(LOCTEXT("PListOpenToolTip", "Open an existing plist file"))
										.OnClicked(this, &SPListEditorPanel::OnOpenClicked)
									]

									// Save Button
									+SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(2, 0)
									.HAlign(HAlign_Left)
									[
										SNew(SButton)
										.Text(LOCTEXT("PListSave", "Save"))
										.ToolTipText(LOCTEXT("PListSaveToolTip", "Save current working plist"))
										.OnClicked(this, &SPListEditorPanel::OnSaveClicked)
									]

									// Save As Button
									+SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(2, 0)
									.HAlign(HAlign_Left)
									[
										SNew(SButton)
										.Text(LOCTEXT("PListSaveAs", "Save As..."))
										.ToolTipText(LOCTEXT("PListSaveAsToolTip", "Save current working plist with a specified filename"))
										.OnClicked(this, &SPListEditorPanel::OnSaveAsClicked)
									]
								]
							]

							// Text to display opened file name
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2, 0)
							[
								SAssignNew(FileNameWidget, SEditableText)
								.Text(FText::FromString(LoadedFileName))
								.IsReadOnly(true)
							]
						]

						// Rows for any extra buttons
						+ SVerticalBox::Slot()
						.Padding(10, 2)
						[
							SNew(SHorizontalBox)

							// Note: Removed this button in lieu of performing most actions with right-clicking
							//// RemoveSelectedRows button
							//+SHorizontalBox::Slot()
							//.Padding(0, 0, 10, 0)
							//.AutoWidth()
							//.HAlign(HAlign_Left)
							//[
							//	SAssignNew(RemoveSelectedButton, SButton)
							//	.Text(LOCTEXT("PListRemoveSelected", "Remove Selected Rows"))
							//	.ToolTipText(LOCTEXT("PListRemoveSelectedToolTip", "Removes the selected rows from the plist below"))
							//	.OnClicked(this, &SPListEditorPanel::OnDeleteSelectedClicked)
							//	.IsEnabled(false)
							//]

							// Search Bar
							+SHorizontalBox::Slot()
							.FillWidth(1)
							[
								SAssignNew(SearchBox, SSearchBox)
								.IsEnabled(this, &SPListEditorPanel::IsSearchBarEnabled)
								.OnTextChanged(this, &SPListEditorPanel::OnFilterTextChanged)
							]
						]
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					// Add a ListView for plist members
					SAssignNew(InternalTree, STreeView<TSharedPtr<IPListNode> >)
					.TreeItemsSource(&PListNodes)
					.SelectionMode(ESelectionMode::Multi)
					.OnContextMenuOpening(this, &SPListEditorPanel::OnContextMenuOpen)
					.OnGetChildren(this, &SPListEditorPanel::OnGetChildren)
					.OnGenerateRow(this, &SPListEditorPanel::OnGenerateRow)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(FName(TEXT("PListKeyColumn")))
						.FillWidth(0.5f)
						.HeaderContentPadding(FMargin(6))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PListKeySectionTitle", "Key"))
						]

						+ SHeaderRow::Column(FName(TEXT("PListValueTypeColumn")))
						.FillWidth(0.1f)
						.HeaderContentPadding(FMargin(6))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PListValueTypeSectionTitle", "Value Type"))
						]

						+ SHeaderRow::Column(FName(TEXT("PListValueColumn")))
						.FillWidth(0.4f)
						.HeaderContentPadding(FMargin(6))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PListValueSectionTitle", "Value"))
						]
					)
				]
			]
		]

		// Notifications
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(15)
		[
			SAssignNew(NotificationListPtr, SNotificationList)
			.Visibility(EVisibility::HitTestInvisible)
		]
	];

	// Default try to load the file GameName-Info.plist file when widget is opened
	FString DefaultFile = FString(FApp::GetProjectName()) + TEXT("-Info.plist");
	InOutLastPath += DefaultFile;
	OpenFile(InOutLastPath);

	// Bind Commands
	BindCommands();
}

/** Helper method to bind commands */
void SPListEditorPanel::BindCommands()
{
	const FPListEditorCommands& Commands = FPListEditorCommands::Get();

	UICommandList->MapAction(
		Commands.NewCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnNew));

	UICommandList->MapAction(
		Commands.OpenCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnOpen));

	UICommandList->MapAction(
		Commands.SaveCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnSave));

	UICommandList->MapAction(
		Commands.SaveAsCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnSaveAs));

	UICommandList->MapAction(
		Commands.DeleteSelectedCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnDeleteSelected),
		FCanExecuteAction::CreateSP(this, &SPListEditorPanel::DetermineDeleteSelectedContext));

	UICommandList->MapAction(
		Commands.MoveUpCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnMoveUp),
		FCanExecuteAction::CreateSP(this, &SPListEditorPanel::DetermineMoveUpContext));

	UICommandList->MapAction(
		Commands.MoveDownCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnMoveDown),
		FCanExecuteAction::CreateSP(this, &SPListEditorPanel::DetermineMoveDownContext));

	UICommandList->MapAction(
		Commands.AddDictionaryCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnAddDictionary),
		FCanExecuteAction::CreateSP(this, &SPListEditorPanel::DetermineAddDictionaryContext));

	UICommandList->MapAction(
		Commands.AddArrayCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnAddArray),
		FCanExecuteAction::CreateSP(this, &SPListEditorPanel::DetermineAddArrayContext));

	UICommandList->MapAction(
		Commands.AddStringCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnAddString),
		FCanExecuteAction::CreateSP(this, &SPListEditorPanel::DetermineAddStringContext));

	UICommandList->MapAction(
		Commands.AddBooleanCommand,
		FExecuteAction::CreateSP(this, &SPListEditorPanel::OnAddBoolean),
		FCanExecuteAction::CreateSP(this, &SPListEditorPanel::DetermineAddBooleanContext));
}

/** Class to create rows for the Internal List using ListView Columns */
class SPListNodeRow : public SMultiColumnTableRow<TSharedPtr<IPListNode> >
{
public:
	SLATE_BEGIN_ARGS( SPListNodeRow )
		{}

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<IPListNode> InItem )
	{
		Item = InItem;

		SMultiColumnTableRow<TSharedPtr<IPListNode> >::Construct( FSuperRowType::FArguments(), InOwnerTable );
	}

	TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName )
	{
		return 
		Item->GenerateWidgetForColumn(ColumnName, Item->GetDepth(), this);
	}

	/** The referenced row item (List Node) */
	TSharedPtr<IPListNode> Item;
};

/** Generates the Row for each member in the ListView */
TSharedRef<ITableRow> SPListEditorPanel::OnGenerateRow( TSharedPtr<IPListNode> InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	// Create a row with or without columns based on item
	if(InItem->UsesColumns())
	{
		return
		SNew(SPListNodeRow, OwnerTable, InItem);
	}
	else
	{
		return InItem->GenerateWidget(OwnerTable);
	}
}

/** Delegate to get the children of the stored items */
void SPListEditorPanel::OnGetChildren(TSharedPtr<IPListNode> InItem, TArray<TSharedPtr<IPListNode> >& OutItems)
{
	OutItems = InItem->GetChildren();
}

/** Helper function to recursively build the nodes for the tree */
bool RecursivelyBuildTree(SPListEditorPanel* EditorPtr, TSharedPtr<IPListNode> ParentNode, const FXmlNode* XmlNode, FString& OutError, int32 ParentDepth, bool bFillingArray = false)
{
	// Null xml node is fine. Simply back out. (Base case)
	if(XmlNode == nullptr)
	{
		return true;
	}

	// Operations:
	// - GetNextNode()			Next element in series
	// - GetFirstChildNode()	Children of element
	// - GetTag()				Tag
	// - GetContent()			Value (if applicable)

	// Get node name
	FString NodeName = XmlNode->GetTag();
	NodeName = NodeName.ToLower();

	// Handle Dictionary Tag
	if(NodeName == TEXT("dict"))
	{
		// Create a dictionary node
		TSharedPtr<FPListNodeDictionary> DictNode = TSharedPtr<FPListNodeDictionary>(new FPListNodeDictionary(EditorPtr));

		// Set whether we are in an array or not
		DictNode->SetArrayMember(bFillingArray);

		// Set Depth
		DictNode->SetDepth(ParentDepth + 1);

		// Recursively fill the dictionary
		if(!RecursivelyBuildTree(EditorPtr, DictNode, XmlNode->GetFirstChildNode(), OutError, ParentDepth + 1))
		{
			return false;
		}

		// Add dictionary to parent
		ParentNode->AddChild(DictNode);

		// Recursively build using the next node in file/array/etc
		if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth, bFillingArray))
		{
			return false;
		}

		return true;
	}

	// Handle Key Tag
	else if(NodeName == TEXT("key"))
	{
		// Save key value
		FString Key = XmlNode->GetContent();

		// Assert that the key actually has a value
		if(!Key.Len())
		{
			OutError = LOCTEXT("PListXMLErrorMissingKeyString", "Error while parsing plist: Key found without a string").ToString();
			return false;
		}

		// Get the next node
		XmlNode = XmlNode->GetNextNode();

		// No value after the key
		if(XmlNode == nullptr)
		{
			OutError = LOCTEXT("PListXMLErrorMissingKeyValue", "Error while parsing plist: Got a key without an associated value").ToString();
			return false;
		}

		// XmlNode Tag should now be String/True/False/Array
		NodeName = XmlNode->GetTag();
		NodeName = NodeName.ToLower();

		// Array Tag
		if(NodeName == TEXT("array"))
		{
			// Create an array node
			TSharedPtr<FPListNodeArray> ArrayNode = TSharedPtr<FPListNodeArray>(new FPListNodeArray(EditorPtr));

			// Set depth
			ArrayNode->SetDepth(ParentDepth + 1);

			// Fill Array Key
			ArrayNode->SetKey(Key);

			// Recursively fill array
			if(!RecursivelyBuildTree(EditorPtr, ArrayNode, XmlNode->GetFirstChildNode(), OutError, ParentDepth + 1, true))
			{
				return false;
			}

			// Add array to parent
			ParentNode->AddChild(ArrayNode);

			// Recursively add elements in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth))
			{
				return false;
			}

			return true;
		}

		// String Tag
		else if(NodeName == TEXT("string"))
		{
			// Create string node
			TSharedPtr<FPListNodeString> StringNode = TSharedPtr<FPListNodeString>(new FPListNodeString(EditorPtr));

			// Set Depth
			StringNode->SetDepth(ParentDepth + 1);

			// Fill out Key
			StringNode->SetKey(Key);

			// Check value
			FString Value = XmlNode->GetContent();
			if(Value.Len() == 0)
			{
				OutError = LOCTEXT("PListXMLErrorNullValueString", "Error while parsing plist: Value found is null (empty string)").ToString();
				return false;
			}

			// Fill out value
			StringNode->SetValue(Value);

			// Add string to parent
			ParentNode->AddChild(StringNode);

			// Recursively add element in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth))
			{
				return false;
			}

			return true;
		}

		// True Tag
		else if(NodeName == TEXT("true"))
		{
			// Create boolean node
			TSharedPtr<FPListNodeBoolean> BooleanNode = TSharedPtr<FPListNodeBoolean>(new FPListNodeBoolean(EditorPtr));

			// Set Depth
			BooleanNode->SetDepth(ParentDepth + 1);

			// Fill out key
			BooleanNode->SetKey(Key);

			// Fill out value
			BooleanNode->SetValue(true);

			// Add boolean to parent
			ParentNode->AddChild(BooleanNode);

			// Recursively add element in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth))
			{
				return false;
			}

			return true;
		}

		// False Tag
		else if(NodeName == TEXT("false"))
		{
			// Create boolean node
			TSharedPtr<FPListNodeBoolean> BooleanNode = TSharedPtr<FPListNodeBoolean>(new FPListNodeBoolean(EditorPtr));

			// Set Depth
			BooleanNode->SetDepth(ParentDepth + 1);

			// Fill out key
			BooleanNode->SetKey(Key);

			// Fill out value
			BooleanNode->SetValue(false);

			// Add boolean to parent
			ParentNode->AddChild(BooleanNode);

			// Recursively add element in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth))
			{
				return false;
			}

			return true;
		}

		// Unexpected/Unimplemented Tag
		else
		{
			OutError = LOCTEXT("PListXMLErrorUnexpectedTag", "Error while parsing plist: Got unexpected XML tag").ToString();
			OutError += FString::Printf(TEXT(" (%s)"), NodeName.GetCharArray().GetData());
			return false;
		}
	}

	// Handle Array Tag
	else if(NodeName == TEXT("array"))
	{
		if(bFillingArray)
		{
			// Create an array node
			TSharedPtr<FPListNodeArray> ArrayNode = TSharedPtr<FPListNodeArray>(new FPListNodeArray(EditorPtr));

			// Set Depth
			ArrayNode->SetDepth(ParentDepth + 1);

			// Fill Array Key
			ArrayNode->SetKey(TEXT("FIXME"));
			ArrayNode->SetArrayMember(true);

			// Recursively fill array
			if(!RecursivelyBuildTree(EditorPtr, ArrayNode, XmlNode->GetFirstChildNode(), OutError, ParentDepth + 1, true))
			{
				return false;
			}

			// Add array to parent
			ParentNode->AddChild(ArrayNode);

			// Recursively add elements in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth, true))
			{
				return false;
			}

			return true;
		}
		else
		{
			// Error: Got array tag without getting a key first.
			OutError = LOCTEXT("PListXMLErrorUnexpectedArray", "Error while parsing plist: Got unexpected array tag without preceeding key").ToString();
			return false;
		}
	}

	// Handle String Tag
	else if(NodeName == TEXT("string"))
	{
		if(bFillingArray)
		{
			// Create string node
			TSharedPtr<FPListNodeString> StringNode = TSharedPtr<FPListNodeString>(new FPListNodeString(EditorPtr));

			// Set Depth
			StringNode->SetDepth(ParentDepth + 1);

			// Fill out Key
			StringNode->SetKey(TEXT("NOKEY"));
			StringNode->SetArrayMember(true);

			// Check value
			FString Value = XmlNode->GetContent();
			if(Value.Len() == 0)
			{
				OutError = LOCTEXT("PListXMLErrorNullValueString", "Error while parsing plist: Value found is null (empty string)").ToString();
				return false;
			}

			// Fill out value
			StringNode->SetValue(Value);

			// Add string to parent
			ParentNode->AddChild(StringNode);

			// Recursively add element in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth, true))
			{
				return false;
			}

			return true;
		}
		else
		{
			// Error: Got string tag without getting a key first.
			OutError = LOCTEXT("PListXMLErrorUnexpectedString", "Error while parsing plist: Got unexpected string tag without preceeding key").ToString();
			return false;
		}
	}

	// Handle True Tag
	else if(NodeName == TEXT("true"))
	{
		if(bFillingArray)
		{
			// Create boolean node
			TSharedPtr<FPListNodeBoolean> BooleanNode = TSharedPtr<FPListNodeBoolean>(new FPListNodeBoolean(EditorPtr));

			// Set Depth
			BooleanNode->SetDepth(ParentDepth + 1);

			// Fill out key
			BooleanNode->SetKey(TEXT("NOKEY"));
			BooleanNode->SetArrayMember(true);

			// Fill out value
			BooleanNode->SetValue(true);

			// Add boolean to parent
			ParentNode->AddChild(BooleanNode);

			// Recursively add element in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth, true))
			{
				return false;
			}

			return true;
		}
		else
		{
			// Error: Got true tag without getting a key first.
			OutError = LOCTEXT("PListXMLErrorUnexpectedTrue", "Error while parsing plist: Got unexpected true tag without preceeding key").ToString();
			return false;
		}
	}

	// Handle False Tag
	else if(NodeName == TEXT("false"))
	{
		if(bFillingArray)
		{
			// Create boolean node
			TSharedPtr<FPListNodeBoolean> BooleanNode = TSharedPtr<FPListNodeBoolean>(new FPListNodeBoolean(EditorPtr));

			// Set Depth
			BooleanNode->SetDepth(ParentDepth + 1);

			// Fill out key
			BooleanNode->SetKey(TEXT("NOKEY"));
			BooleanNode->SetArrayMember(true);

			// Fill out value
			BooleanNode->SetValue(false);

			// Add boolean to parent
			ParentNode->AddChild(BooleanNode);

			// Recursively add element in list as necessary
			if(!RecursivelyBuildTree(EditorPtr, ParentNode, XmlNode->GetNextNode(), OutError, ParentDepth + 1, true))
			{
				return false;
			}

			return true;
		}
		else
		{
			// Error: Got false tag without getting a key first.
			OutError = LOCTEXT("PListXMLErrorUnexpectedFalse", "Error while parsing plist: Got unexpected false tag without preceeding key").ToString();
			return false;
		}
	}

	// Unrecognized/Unsupported Tag (eg. date, real, integer, data, etc)
	else
	{
		OutError = LOCTEXT("PListXMLErrorUnexpectedTag", "Error while parsing plist: Got unexpected XML tag").ToString();
		OutError += FString::Printf(TEXT(" (%s)"), NodeName.GetCharArray().GetData());
		return false;
	}
}

/** Helper method to parse through and load an Xml tree into an internal intermediate format for Slate */
bool SPListEditorPanel::ParseXmlTree(FXmlFile& Doc, FString& OutError)
{
	// Check for bad document
	if(!Doc.IsValid())
	{
		OutError = Doc.GetLastError();
		return false;
	}

	// Empty old contents
	PListNodes.Empty();

	// Create root FPListNodeFile
	TSharedPtr<FPListNodeFile> FileNode = TSharedPtr<FPListNodeFile>(new FPListNodeFile(this));

	// Set Depth
	FileNode->SetDepth(0);

	// Get the working Xml Node
	const FXmlNode* XmlRoot = Doc.GetRootNode();
	if(!XmlRoot)
	{
		return false;
	}
	FString RootName = XmlRoot->GetTag();
	if(RootName != TEXT("plist"))
	{
		return false;
	}
	XmlRoot = XmlRoot->GetFirstChildNode();

	// Recursively build the tree
	FString ErrorMessage;
	bool bSuccess = RecursivelyBuildTree(this, FileNode, XmlRoot, ErrorMessage, 0);

	// Back out if we fail
	if(!bSuccess)
	{
		OutError = ErrorMessage;
		return false;
	}

	// Add file to internal nodes
	PListNodes.Add(FileNode);

	// Update everything
	FileNode->Refresh();

	// All done
	return true;
}

/** Handles when the tab is trying to be closed (prompt saving if necessary) Returning false will prevent tab from closing */
bool SPListEditorPanel::OnTabClose()
{
	// Nothing loaded, close
	if(!bNewFile && !bFileLoaded)
	{
		return true;
	}

	// Don't bother if we're not dirty
	if(!bDirty)
	{
		return true;
	}

	// Prompt user to save
	return PromptSave();
}

/** Delegate for New ui command */
void SPListEditorPanel::OnNew()
{
	OnNewClicked();
}

/** Delegate to create a new plist when the button is clicked */
FReply SPListEditorPanel::OnNewClicked()
{
	// Prompt save if dirty
	if(bDirty)
	{
		if(!PromptSave())
		{
			return FReply::Handled();
		}
	}

	// Empty old stuff
	PListNodes.Empty();

	// Add a new file to the list and that's it!
	TSharedPtr<FPListNodeFile> FileNode = TSharedPtr<FPListNodeFile>(new FPListNodeFile(this));
	FileNode->SetDepth(0);
	FileNode->Refresh();
	PListNodes.Add(FileNode);

	// Regenerate Tree Widget
	InternalTree->RequestTreeRefresh();

	// Set some flags and other misc
	InOutLastPath = FPaths::ProjectDir() + TEXT("Build/IOS/UnnamedPList");
	bFileLoaded = false;
	bPromptSave = false;
	bNewFile = true;
	bPromptDelete = true;
	MarkDirty();

	return FReply::Handled();
}

/** Helper function to open a file */
bool SPListEditorPanel::OpenFile(FString FilePath)
{
	// User successfully chose a file; remember the path for the next time the dialog opens.
	InOutLastPath = FilePath;

	// Try to load the file
	FXmlFile Doc;
	bool bLoadResult = Doc.LoadFile(FilePath);

	FString OutError;
	if( !bLoadResult )
	{
		// try Info.plist
		FilePath = FilePath.Replace(*(FString(FApp::GetProjectName()) + TEXT("-")), TEXT(""));
		bLoadResult = Doc.LoadFile(FilePath);
	}

	if( !bLoadResult || !ParseXmlTree(Doc, OutError) )
	{
		DisplayNotification( LOCTEXT("PListLoadFail", "Load Failed"), NTF_Fail );

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Filepath"), FText::FromString( InOutLastPath ));
		Arguments.Add(TEXT("ErrorDetails"), FText::FromString( OutError ));
		FText ErrorMessageFormatting = LOCTEXT("PListXMLLoadErrorFormatting", "Failed to Load PList File: {Filepath}\n\n{ErrorDetails}");
		FText ErrorMessage = FText::Format( ErrorMessageFormatting, Arguments );

		FText ErrorTitle = LOCTEXT("PListLoadFailDialogCaption", "Error");

		// Show error message
		FMessageDialog::Open( EAppMsgType::Ok, ErrorMessage, ErrorTitle );
		return false;
	}
	else
	{
		// Change file name to match loaded file
		FileNameWidget->SetText(FText::FromString(InOutLastPath));
		bFileLoaded = true;
		bPromptSave = true;
		bNewFile = false;
		bPromptDelete = true;

		// Expand all items
		class TreeExpander
		{
		public:
			void ExpandRecursively(TSharedPtr<STreeView<TSharedPtr<IPListNode> > > TreeWidget, TSharedPtr<IPListNode> In)
			{
				if(In.Get())
				{
					TreeWidget->SetItemExpansion(In, true);
					auto Children = In->GetChildren();
					for(int32 i = 0; i < Children.Num(); ++i)
					{
						ExpandRecursively(TreeWidget, Children[i]);
					}
				}
			}
		} TreeExpander;
		TreeExpander.ExpandRecursively(InternalTree, PListNodes[0]);

		// Regenerate List Widget
		InternalTree->RequestTreeRefresh();

		// Show notification
		DisplayNotification(LOCTEXT("PListLoadSuccess", "Load Successful"), NTF_Success);
		ClearDirty();
		return true;
	}
}

/** Delegate for Open ui command */
void SPListEditorPanel::OnOpen()
{
	OnOpenClicked();
}

/** Delegate to open an existing plist when the button is clicked */
FReply SPListEditorPanel::OnOpenClicked()
{
	// Prompt save before going to open dialog if dirty
	if(bDirty)
	{
		if(!PromptSave())
		{
			return FReply::Handled();
		}
	}

	// Open file browser to get a file to load
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	TArray<FString> OutOpenFilenames;
	if ( DesktopPlatform )
	{
		//FString DefaultPath = FPaths::ProjectDir() + TEXT("Build/IOS/");
		//FString DefaultFile = FString(FApp::GetProjectName()) + TEXT("-Info.plist");
		FString FileTypes = TEXT("Property List (*.plist)|*.plist|All Files (*.*)|*.*");
		DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("PListOpenDialogTitle", "Open").ToString(),
			InOutLastPath,
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OutOpenFilenames
		);
	}

	if ( OutOpenFilenames.Num() > 0 )
	{
		OpenFile(OutOpenFilenames[0]);
	}

	return FReply::Handled();
}

/** Helper function to check if PListNodes is valid (every element being valid) */
bool SPListEditorPanel::ValidatePListNodes()
{
	// All nodes must be valid for validation to pass
	for(int32 i = 0; i < PListNodes.Num(); ++i)
	{
		if( !PListNodes[i]->IsValid() )
		{
			return false;
		}
	}
	return true;
}

/** Helper method to write out the contents of PListNodes using a valid FileWriter */
void SPListEditorPanel::SerializePListNodes(FArchive* Writer)
{
	check(Writer != NULL);

	// Get XML from nodes
	// Note: Assuming there is 1 node in the list, which should always be a file. All other nodes are children of the file
	check(PListNodes.Num() == 1);
	check(PListNodes[0]->GetType() == EPLNTypes::PLN_File);
	FString XMLOutput = PListNodes[0]->ToXML();

	// Write Data
	Writer->Serialize(TCHAR_TO_UTF8(*XMLOutput), XMLOutput.Len());
}

/** Helper method to check if a file exists */
bool SPListEditorPanel::CheckFileExists(FString Path)
{
	FArchive* FileReader = IFileManager::Get().CreateFileReader(*Path);
	if(FileReader)
	{
		FileReader->Close();
		delete FileReader;
		return true;
	}
	else
	{
		return false;
	}
}

/** Helper method to prompt the user to delete element(s) */
bool SPListEditorPanel::PromptDelete()
{
	if(!bPromptDelete)
	{
		return true;
	}

	FText DeleteMessage = LOCTEXT("PListDeleteConfirmation", "Are you sure you want to remove the selected entries? (This action is irreversible!)");
	FText DeleteTitle = LOCTEXT("PListDeleteConfirmationCaption", "Confirm Removal");
	EAppReturnType::Type ret = FMessageDialog::Open( EAppMsgType::YesNoYesAll, DeleteMessage, DeleteTitle );
	if(ret == EAppReturnType::Yes)
	{
		return true;
	}
	else if(ret == EAppReturnType::No)
	{
		return false;
	}
	else if(ret == EAppReturnType::YesAll)
	{
		bPromptDelete = false;
		return true;
	}
	else
	{
		return false;
	}
}

/** Helper method to perform a save prompt. Returns true if the caller can pass the prompt and false if the caller should not continue its routine */
bool SPListEditorPanel::PromptSave()
{
	// Prompt user to save
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("FilePath"), FText::FromString( InOutLastPath ));
	FText DialogText = FText::Format( LOCTEXT("PListCloseTabSaveTextFormatting", "Save {FilePath}?"), Arguments );
	FText DialogTitle = LOCTEXT("PListCloseTabSaveCaption", "Save");

	EAppReturnType::Type ret = FMessageDialog::Open(EAppMsgType::YesNoCancel, DialogText, DialogTitle );
	if(ret == EAppReturnType::Yes)
	{
		// Get the saving location if necessary (like on new files)
		if(bNewFile)
		{
			// Get the saving location
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			TArray<FString> OutFilenames;
			if (DesktopPlatform)
			{
				FString FileTypes = TEXT("Property List (*.plist)|*.plist|All Files (*.*)|*.*");
				DesktopPlatform->SaveFileDialog(
					FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
					LOCTEXT("PListSaveDialogTitle", "Save").ToString(),
					InOutLastPath,
					TEXT(""),
					FileTypes,
					EFileDialogFlags::None,
					OutFilenames
				);
			}

			if (OutFilenames.Num() > 0)
			{
				// User successfully chose a file, but may not have entered a file extension
				FString OutFilename = OutFilenames[0];
				if( !OutFilename.EndsWith(TEXT(".plist")) )
				{
					OutFilename += TEXT(".plist");

					// Prompt overwriting existing file (only if a file extension was not originally provided since windows browser detects that case)
					if(CheckFileExists(OutFilename))
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("Filename"), FText::FromString(OutFilename));
						FText OverwriteMessageFormatting = LOCTEXT("PListCloseTabOverwriteTextFormatting", "Overwrite existing file {Filename}?");
						FText OverwriteDialogText = FText::Format(OverwriteMessageFormatting, Args);
						FText OverwriteDialogTitle = LOCTEXT("PListWarningCaption", "Warning");

						EAppReturnType::Type RetVal = FMessageDialog::Open(EAppMsgType::YesNo, OverwriteDialogText, OverwriteDialogTitle);
						if(RetVal != EAppReturnType::Yes)
						{
							// Said not to overwrite (or clicked x) so bail out
							return false;
						}
					}
				}

				// Remember path for next time
				InOutLastPath = OutFilename;
			}
			else
			{
				// No file chosen, so interpret it as a cancel
				return false;
			}
		}

		// Open a file for writing
		FString OutPath = InOutLastPath;

		// Make sure there are no invalid members before saving
		if(!ValidatePListNodes())
		{
			DisplayNotification(LOCTEXT("PListSaveFail", "Save Failed"), NTF_Fail);

			// Display Message
			FText ValidationFailMessage = LOCTEXT("PListNodeValidationFail", "Cannot save file: Not all plist entries have valid input");
			FText Title = LOCTEXT("PListWarningCaption", "Warning");
			FMessageDialog::Open( EAppMsgType::Ok, ValidationFailMessage, Title );

			// Cancel
			return false;
		}

		// Open file for writing
		FArchive* OutputFile = IFileManager::Get().CreateFileWriter(*OutPath, FILEWRITE_EvenIfReadOnly);
		checkSlow(OutputFile != NULL);

		if(OutputFile)
		{
			SerializePListNodes(OutputFile);

			// Finished
			OutputFile->Close();
			delete OutputFile;

			// Change status flags
			FileNameWidget->SetText(FText::FromString(InOutLastPath));
			bNewFile = false;
			bFileLoaded = true;
			bPromptSave = false;

			// Show Notification
			DisplayNotification(LOCTEXT("PListSaveSuccess", "Save Successful"), NTF_Success);
			ClearDirty();
		}

		// Can continue
		return true;
	}
	else if(ret == EAppReturnType::No)
	{
		// Can continue
		return true;
	}
	else
	{
		// Don't continue
		return false;
	}
}

/** Delegate for Save ui command */
void SPListEditorPanel::OnSave()
{
	OnSaveClicked();
}

/** Delegate to save the working plist when the button is clicked */
FReply SPListEditorPanel::OnSaveClicked()
{
	// Nothing loaded, return
	if(!bNewFile && !bFileLoaded)
	{
		return FReply::Handled();
	}

	// Prompt overwriting saving if necessary
	if(bPromptSave)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("FilePath"), FText::FromString( InOutLastPath ));
		FText DialogText = FText::Format( LOCTEXT("PListOverwriteMessageFormatting", "Overwrite {FilePath}?"), Arguments );
		FText DialogTitle = LOCTEXT("PListOverwriteCaption", "Warning");

		EAppReturnType::Type ret = FMessageDialog::Open( EAppMsgType::YesNo, DialogText, DialogTitle );
		if(ret != EAppReturnType::Yes)
		{
			return FReply::Handled();
		}
	}

	// Get the saving location if necessary (like on new files)
	if(bNewFile)
	{
		// Get the saving location
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		TArray<FString> OutFilenames;
		if (DesktopPlatform)
		{
			FString FileTypes = TEXT("Property List (*.plist)|*.plist|All Files (*.*)|*.*");
			DesktopPlatform->SaveFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
				LOCTEXT("PListSaveDialogTitle", "Save").ToString(),
				InOutLastPath,
				TEXT(""),
				FileTypes,
				EFileDialogFlags::None,
				OutFilenames
			);
		}

		if (OutFilenames.Num() > 0)
		{
			// User successfully chose a file, but may not have entered a file extension
			FString OutFilename = OutFilenames[0];
			if( !OutFilename.EndsWith(TEXT(".plist")) )
			{
				OutFilename += TEXT(".plist");

				// Prompt overwriting existing file (only if a file extension was not originally provided since windows browser detects that case)
				if(CheckFileExists(OutFilename))
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("Filename"), FText::FromString( OutFilename ));
					FText OverwriteMessageFormatting = LOCTEXT("PListFileExistsMessageFormatting", "Overwrite existing file {Filename}?");
					FText DialogText = FText::Format( OverwriteMessageFormatting, Arguments );
					FText DialogTitle = LOCTEXT("PListWarningCaption", "Warning");

					EAppReturnType::Type RetVal = FMessageDialog::Open( EAppMsgType::YesNo, DialogText, DialogTitle );
					if(RetVal != EAppReturnType::Yes)
					{
						// Said not to overwrite (or clicked x) so bail out
						return FReply::Handled();
					}
				}
			}

			// Remember path for next time
			InOutLastPath = OutFilename;
		}
		else
		{
			// No file chosen, so do nothing
			return FReply::Handled();
		}
	}

	// Open a file for writing
	FString OutPath = InOutLastPath;

	// Make sure there are no invalid members before saving
	if(!ValidatePListNodes())
	{
		DisplayNotification(LOCTEXT("PListSaveFail", "Save Failed"), NTF_Fail);

		// Display Message
		FText OverwriteMessage = LOCTEXT("PListNodeValidationFail", "Cannot save file: Not all plist entries have valid input");
		FText DialogTitle = LOCTEXT("PListWarningCaption", "Warning");
		FMessageDialog::Open(EAppMsgType::Ok, OverwriteMessage, DialogTitle );

		return FReply::Handled();
	}

	// Open file for writing
	FArchive* OutputFile = IFileManager::Get().CreateFileWriter(*OutPath, FILEWRITE_EvenIfReadOnly);
	checkSlow(OutputFile != NULL);

	if(OutputFile)
	{
		SerializePListNodes(OutputFile);

		// Finished
		OutputFile->Close();
		delete OutputFile;

		// Change status flags
		FileNameWidget->SetText(FText::FromString(InOutLastPath));
		bNewFile = false;
		bFileLoaded = true;
		bPromptSave = false;

		// Show Notification
		DisplayNotification(LOCTEXT("PListSaveSuccess", "Save Successful"), NTF_Success);
		ClearDirty();
	}

	return FReply::Handled();
}

/** Delegate for SaveAs ui command */
void SPListEditorPanel::OnSaveAs()
{
	OnSaveAsClicked();
}

/** Delegate to save the working plist with a specified name when the button is clicked */
FReply SPListEditorPanel::OnSaveAsClicked()
{
	// Nothing loaded, return
	if(!bNewFile && !bFileLoaded)
		return FReply::Handled();

	// Get the saving location
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	TArray<FString> OutFilenames;
	if (DesktopPlatform)
	{
		FString FileTypes = TEXT("Property List (*.plist)|*.plist|All Files (*.*)|*.*");
		DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("PListSaveAsDialogTitle", "Save As").ToString(),
			InOutLastPath,
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OutFilenames
		);
	}

	if (OutFilenames.Num() > 0)
	{
		// User successfully chose a file, but may not have entered a file extension
		FString OutFilename = OutFilenames[0];
		if( !OutFilename.EndsWith(TEXT(".plist")) )
		{
			OutFilename += TEXT(".plist");

			// Prompt overwriting existing file (only if a file extension was not originally provided since windows browser detects that case)
			if(CheckFileExists(OutFilename))
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Filename"), FText::FromString( InOutLastPath ));
				FText OverwriteMessageFormatting = LOCTEXT("PListFileExistsMessageFormatting", "Overwrite existing file {Filename}?");
				FText DialogText = FText::Format( OverwriteMessageFormatting, Arguments );
				FText DialogTitle = LOCTEXT("PListWarningCaption", "Warning");

				EAppReturnType::Type RetVal = FMessageDialog::Open( EAppMsgType::YesNo, DialogText, DialogTitle );
				if(RetVal != EAppReturnType::Yes)
				{
					// Said not to overwrite (or clicked x) so bail out
					return FReply::Handled();
				}
			}
		}

		// Remember path for next time
		InOutLastPath = OutFilename;
	}
	else
	{
		// No file chosen, so do nothing
		return FReply::Handled();
	}

	// Make sure there are no invalid members before saving
	if(!ValidatePListNodes())
	{
		DisplayNotification(LOCTEXT("PListSaveFail", "Save Failed"), NTF_Fail);

		// Display Message
		FText OverwriteMessage = LOCTEXT("PListNodeValidationFail", "Cannot save file: Not all plist entries have valid input");
		FText DialogTitle = LOCTEXT("PListWarningCaption", "Warning");

		FMessageDialog::Open( EAppMsgType::Ok, OverwriteMessage, DialogTitle );

		return FReply::Handled();
	}

	// Open a file for writing
	FString OutPath = InOutLastPath;

	// Open file for writing
	FArchive* OutputFile = IFileManager::Get().CreateFileWriter(*OutPath, FILEWRITE_EvenIfReadOnly);

	if(OutputFile)
	{
		SerializePListNodes(OutputFile);

		// Finished
		OutputFile->Close();
		delete OutputFile;

		// Change status flags and misc
		FileNameWidget->SetText(FText::FromString(InOutLastPath));
		bNewFile = false;
		bFileLoaded = true;
		bPromptSave = false;

		// Show Notification
		DisplayNotification(LOCTEXT("PListSaveSuccess", "Save Successful"), NTF_Success);
		ClearDirty();
	}

	// File couldn't be opened
	else
	{
		// Tried to open a file for saving that was invalid (such as bad characters, too long path, etc)

		// SHOULD never happen since we pick a file from the browser
		check(!TEXT("Opening file to read failed which should never happen!"));
	}

	return FReply::Handled();
}

/** Helper routine to recursively search through children to find a node */
static bool FindParentRecursively(const TSharedPtr<IPListNode>& InParent, const TSharedPtr<IPListNode>& InChildNode, TSharedPtr<IPListNode>& OutFoundNode)
{
	auto Children = InParent->GetChildren();

	for(int32 i = 0; i < Children.Num(); ++i)
	{
		if(Children[i] == InChildNode)
		{
			OutFoundNode = InParent;
			return true;
		}
		else
		{
			bool bFound = FindParentRecursively(Children[i], InChildNode, OutFoundNode);
			if(bFound)
			{
				return true;
			}
		}
	}

	return false;
}

/** Helper function to search through nodes to find a specific node's parent */
bool SPListEditorPanel::FindParent(const TSharedPtr<IPListNode>& InChildNode, TSharedPtr<IPListNode>& OutFoundNode) const
{
	// Get the start of the file
	check(PListNodes.Num() == 1);
	TSharedPtr<IPListNode> Head = PListNodes[0];

	// Find the children recursively
	return FindParentRecursively(Head, InChildNode, OutFoundNode);
}

/** Delegate for DeleteSelected ui command */
void SPListEditorPanel::OnDeleteSelected()
{
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() == 0)
	{
		return;
	}

	// Can only delete if we have items selected that are not the top file
	bool bGoodToContinue = false;
	for(int32 i = 0; i < SelectedNodes.Num(); ++i)
	{
		if(SelectedNodes[i]->GetType() != EPLNTypes::PLN_File)
		{
			bGoodToContinue = true;
			break;
		}
	}
	if(bGoodToContinue)
	{
		// Prompt delete
		if(!PromptDelete())
		{
			return;
		}

		// Delete items in order.
		// (The returned list of selected nodes is assumed to be random)
		for(int32 i = 0; i < SelectedNodes.Num(); ++i)
		{
			TSharedPtr<IPListNode> SelectedNode = SelectedNodes[i];

			// Ignore node if it's the file
			if(SelectedNode->GetType() == EPLNTypes::PLN_File)
			{
				continue;
			}

			// Get parent of node to delete
			TSharedPtr<IPListNode> ParentNode;
			bool bFound = FindParent(SelectedNode, ParentNode);

			// If the parent is not found, we can assume that we deleted the parent in a previous iteration.
			// This also means that all children of that parent were deleted (ie: this node)
			if(!bFound)
			{
				continue;
			}

			// Get list of the parent's children
			TArray<TSharedPtr<IPListNode> >& ChildrenList = ParentNode->GetChildren();

			// Delete the child from the parent's list
			ChildrenList.Remove(SelectedNode);

			// Refresh display
			check(InternalTree.IsValid());
			InternalTree->RequestTreeRefresh();
			PListNodes[0]->Refresh();

			MarkDirty();
		}
	}
}

/** Delegate that determines when the delete selected context button can be clicked */
bool SPListEditorPanel::DetermineDeleteSelectedContext() const
{
	// Can only delete if we have a selection of any number of items and one of them is not a file
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();

	if(SelectedNodes.Num() == 0)
	{
		return false;
	}
	else
	{
		for(int32 i = 0; i < SelectedNodes.Num(); ++i)
		{
			if(SelectedNodes[i]->GetType() != EPLNTypes::PLN_File)
			{
				return true;
			}
		}

		return false;
	}
}

/** Delegate for MoveUp command */
void SPListEditorPanel::OnMoveUp()
{
	// Can only move 1 thing at a time
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() == 1)
	{
		TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

		// Ignore the node if it is a file
		if(SelectedNode->GetType() == EPLNTypes::PLN_File)
		{
			return;
		}

		// Get the parent node of selection
		TSharedPtr<IPListNode> Parent;
		bool bFound = FindParent(SelectedNode, Parent);
		if(bFound)
		{
			// Find the child in the parent's children list
			TArray<TSharedPtr<IPListNode> >& ChildList = Parent->GetChildren();
			int32 ListIndex = -1;
			for(int32 i = 0; i < ChildList.Num(); ++i)
			{
				if(ChildList[i] == SelectedNode)
				{
					ListIndex = i;
					break;
				}
			}
			check(ListIndex != -1);

			// Can only move up if we're not the first in the list
			if(ListIndex > 0)
			{
				// Remove child from the parent's list
				ChildList.RemoveAt(ListIndex);

				// Reinsert child at 1 before its position
				ChildList.Insert(SelectedNode, ListIndex - 1);

				// Refresh tree and children
				InternalTree->RequestTreeRefresh();
				Parent->Refresh();

				MarkDirty();
			}
		}
	}
}

/** Delegate for determining when MoveUp can be used */
bool SPListEditorPanel::DetermineMoveUpContext() const
{
	// Can only move 1 thing at a time
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() != 1)
	{
		return false;
	}
	
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

	// Files cannot be contained in lists
	if(SelectedNode->GetType() == EPLNTypes::PLN_File)
	{
		return false;
	}

	// Can only move child up if it's not the first in its parent list
	else
	{
		// Get the parent node of selection
		TSharedPtr<IPListNode> Parent;
		bool bFound = FindParent(SelectedNode, Parent);
		check(bFound);
		if(bFound)
		{
			// Find the child in the parent's children list
			auto ChildList = Parent->GetChildren();
			int32 ListIndex = -1;
			for(int32 i = 0; i < ChildList.Num(); ++i)
			{
				if(ChildList[i] == SelectedNode)
				{
					ListIndex = i;
					break;
				}
			}
			check(ListIndex != -1);

			if(ListIndex > 0)
			{
				return true;
			}
			else
			{
				return false;
			}
		}

		return false;
	}
}

/** Delegate for MoveDown command */
void SPListEditorPanel::OnMoveDown()
{
	// Can only move 1 thing at a time
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() == 1)
	{
		TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

		// Ignore the node if it is a file
		if(SelectedNode->GetType() == EPLNTypes::PLN_File)
		{
			return;
		}

		// Get the parent node of selection
		TSharedPtr<IPListNode> Parent;
		bool bFound = FindParent(SelectedNode, Parent);
		if(bFound)
		{
			// Find the child in the parent's children list
			TArray<TSharedPtr<IPListNode> >& ChildList = Parent->GetChildren();
			int32 ListIndex = -1;
			for(int32 i = 0; i < ChildList.Num(); ++i)
			{
				if(ChildList[i] == SelectedNode)
				{
					ListIndex = i;
					break;
				}
			}
			check(ListIndex != -1);

			// Can only move down if we're not the last in the list
			if(ListIndex < ChildList.Num() - 1)
			{
				// Remove child from the parent's list
				ChildList.RemoveAt(ListIndex);

				// Reinsert child at 1 before its position
				ChildList.Insert(SelectedNode, ListIndex + 1);

				// Refresh tree and children
				InternalTree->RequestTreeRefresh();
				Parent->Refresh();

				MarkDirty();
			}
		}
	}
}

/** Delegate for determining when MoveDown can be used */
bool SPListEditorPanel::DetermineMoveDownContext() const
{
	// Can only move 1 thing at a time
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() != 1)
	{
		return false;
	}
	
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

	// Files cannot be contained in lists
	if(SelectedNode->GetType() == EPLNTypes::PLN_File)
	{
		return false;
	}

	// Can only move child down if it's not the last in its parent list
	else
	{
		// Get the parent node of selection
		TSharedPtr<IPListNode> Parent;
		bool bFound = FindParent(SelectedNode, Parent);
		check(bFound);
		if(bFound)
		{
			// Find the child in the parent's children list
			auto ChildList = Parent->GetChildren();
			int32 ListIndex = -1;
			for(int32 i = 0; i < ChildList.Num(); ++i)
			{
				if(ChildList[i] == SelectedNode)
				{
					ListIndex = i;
					break;
				}
			}
			check(ListIndex != -1);

			if(ListIndex < ChildList.Num() - 1)
			{
				return true;
			}
			else
			{
				return false;
			}
		}

		return false;
	}
}

/** Delegate for adding a dictionary */
void SPListEditorPanel::OnAddDictionary()
{
	// Can only add if we have 1 thing selected
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() != 1)
	{
		return;
	}
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

	// Can only add if the selected node supports dictionary children (ie: file/array)
	if(SelectedNode->GetType() == EPLNTypes::PLN_File ||
		SelectedNode->GetType() == EPLNTypes::PLN_Array)
	{
		// Create a new dictionary
		TSharedPtr<FPListNodeDictionary> DictNode = TSharedPtr<FPListNodeDictionary>(new FPListNodeDictionary(this));
		DictNode->SetArrayMember(SelectedNode->GetType() == EPLNTypes::PLN_Array ? true : false);
		DictNode->SetDepth(SelectedNode->GetDepth() + 1);

		// Add dict to parent
		SelectedNode->AddChild(DictNode);

		// Update the filter for all children
		check(SearchBox.IsValid());
		SelectedNode->OnFilterTextChanged(SearchBox->GetText().ToString());

		// Update the list and children
		check(InternalTree.IsValid());
		InternalTree->RequestTreeRefresh();
		SelectedNode->Refresh();
		InternalTree->SetItemExpansion(SelectedNode, true);

		MarkDirty();
	}
}

/** Delegate for determining when AddDictionary can be used */
bool SPListEditorPanel::DetermineAddDictionaryContext() const
{
	bool bAbleToAdd = true;

	// Can only add if we have 1 selected item
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	bAbleToAdd = SelectedNodes.Num() == 1;

	// Early out
	if(SelectedNodes.Num() == 0)
	{
		return false;
	}

	// Can also only add if the selected node supports children
	// ie: file/array
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];
	check(SelectedNode.IsValid());
	if(SelectedNode->GetType() != EPLNTypes::PLN_File &&
		SelectedNode->GetType() != EPLNTypes::PLN_Array)
	{
		bAbleToAdd = false;
	}

	return bAbleToAdd;
}

/** Delegate for adding a string */
void SPListEditorPanel::OnAddString()
{
	// Can only add if we have 1 thing selected
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() != 1)
	{
		return;
	}
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

	// Can only add if the selected node supports string children (ie: file/array/dict)
	if(SelectedNode->GetType() == EPLNTypes::PLN_File ||
		SelectedNode->GetType() == EPLNTypes::PLN_Dictionary ||
		SelectedNode->GetType() == EPLNTypes::PLN_Array)
	{
		// Create a new string
		TSharedPtr<FPListNodeString> StringNode = TSharedPtr<FPListNodeString>(new FPListNodeString(this));
		StringNode->SetArrayMember(SelectedNode->GetType() == EPLNTypes::PLN_Array ? true : false);
		StringNode->SetDepth(SelectedNode->GetDepth() + 1);
		StringNode->SetKey(TEXT(""));
		StringNode->SetValue(TEXT(""));

		// Add string to parent
		SelectedNode->AddChild(StringNode);

		// Update the filter for all children
		check(SearchBox.IsValid());
		SelectedNode->OnFilterTextChanged(SearchBox->GetText().ToString());

		// Update the list and children
		check(InternalTree.IsValid());
		InternalTree->RequestTreeRefresh();
		SelectedNode->Refresh();
		InternalTree->SetItemExpansion(SelectedNode, true);

		MarkDirty();
	}
}

/** Delegate for determining when AddString can be used */
bool SPListEditorPanel::DetermineAddStringContext() const
{
	bool bAbleToAdd = true;

	// Can only add if we have 1 selected item
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	bAbleToAdd = SelectedNodes.Num() == 1;

	// Early out
	if(SelectedNodes.Num() == 0)
	{
		return false;
	}

	// Can also only add if the selected node supports children
	// ie: file/array/dict
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];
	check(SelectedNode.IsValid());
	if(SelectedNode->GetType() != EPLNTypes::PLN_File &&
		SelectedNode->GetType() != EPLNTypes::PLN_Dictionary &&
		SelectedNode->GetType() != EPLNTypes::PLN_Array)
	{
		bAbleToAdd = false;
	}

	return bAbleToAdd;
}

/** Delegate for adding a boolean */
void SPListEditorPanel::OnAddBoolean()
{
	// Can only add if we have 1 thing selected
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() != 1)
	{
		return;
	}
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

	// Can only add if the selected node supports bool children (ie: file/array/dict)
	if(SelectedNode->GetType() == EPLNTypes::PLN_File ||
		SelectedNode->GetType() == EPLNTypes::PLN_Dictionary ||
		SelectedNode->GetType() == EPLNTypes::PLN_Array)
	{
		// Create a new bool
		TSharedPtr<FPListNodeBoolean> BooleanNode = TSharedPtr<FPListNodeBoolean>(new FPListNodeBoolean(this));
		BooleanNode->SetArrayMember(SelectedNode->GetType() == EPLNTypes::PLN_Array ? true : false);
		BooleanNode->SetDepth(SelectedNode->GetDepth() + 1);
		BooleanNode->SetKey(TEXT(""));
		BooleanNode->SetValue(false);

		// Add bool to parent
		SelectedNode->AddChild(BooleanNode);

		// Update the filter for all children
		check(SearchBox.IsValid());
		SelectedNode->OnFilterTextChanged(SearchBox->GetText().ToString());

		// Update the list and children
		check(InternalTree.IsValid());
		InternalTree->RequestTreeRefresh();
		SelectedNode->Refresh();
		InternalTree->SetItemExpansion(SelectedNode, true);

		MarkDirty();
	}
}

/** Delegate for determining when AddBoolean can be used */
bool SPListEditorPanel::DetermineAddBooleanContext() const
{
	bool bAbleToAdd = true;

	// Can only add if we have 1 selected item
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	bAbleToAdd = SelectedNodes.Num() == 1;

	// Early out
	if(SelectedNodes.Num() == 0)
	{
		return false;
	}

	// Can also only add if the selected node supports children
	// ie: file/array/dict
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];
	check(SelectedNode.IsValid());
	if(SelectedNode->GetType() != EPLNTypes::PLN_File &&
		SelectedNode->GetType() != EPLNTypes::PLN_Dictionary &&
		SelectedNode->GetType() != EPLNTypes::PLN_Array)
	{
		bAbleToAdd = false;
	}

	return bAbleToAdd;
}

/** Delegate to add a new array to the plist as a child of the selected node */
void SPListEditorPanel::OnAddArray()
{
	// Can only add if we have 1 thing selected
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	if(SelectedNodes.Num() != 1)
	{
		return;
	}
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];

	// Can only add if the selected node supports array children (ie: file/dict)
	if(SelectedNode->GetType() == EPLNTypes::PLN_File ||
		SelectedNode->GetType() == EPLNTypes::PLN_Dictionary)
	{
		// Create a new array
		TSharedPtr<FPListNodeArray> ArrayNode = TSharedPtr<FPListNodeArray>(new FPListNodeArray(this));
		ArrayNode->SetArrayMember(SelectedNode->GetType() == EPLNTypes::PLN_Array ? true : false);
		ArrayNode->SetDepth(SelectedNode->GetDepth() + 1);
		ArrayNode->SetKey(TEXT(""));

		// Add string to parent
		SelectedNode->AddChild(ArrayNode);

		// Update the filter for all children
		check(SearchBox.IsValid());
		SelectedNode->OnFilterTextChanged(SearchBox->GetText().ToString());

		// Update the list and children
		check(InternalTree.IsValid());
		InternalTree->RequestTreeRefresh();
		SelectedNode->Refresh();
		InternalTree->SetItemExpansion(SelectedNode, true);

		MarkDirty();
	}
}

/** Delegate that determines when the AddArray button can be clicked in the context menu */
bool SPListEditorPanel::DetermineAddArrayContext() const
{
	bool bAbleToAdd = true;

	// Can only add if we have 1 selected item
	check(InternalTree.IsValid());
	TArray<TSharedPtr<IPListNode> > SelectedNodes = InternalTree->GetSelectedItems();
	bAbleToAdd = SelectedNodes.Num() == 1;

	// Early out
	if(SelectedNodes.Num() == 0)
	{
		return false;
	}

	// Can also only add if the selected node supports children
	// ie: file/dict
	TSharedPtr<IPListNode> SelectedNode = SelectedNodes[0];
	check(SelectedNode.IsValid());
	if(SelectedNode->GetType() != EPLNTypes::PLN_File &&
		SelectedNode->GetType() != EPLNTypes::PLN_Dictionary)
	{
		bAbleToAdd = false;
	}

	return bAbleToAdd;
}

/** Callback for keyboard shortcut commands */
FReply SPListEditorPanel::OnKeyDown( const FGeometry& /*MyGeometry*/, const FKeyEvent& InKeyEvent )
{
	// Perform commands if necessary
	FReply Reply = FReply::Unhandled();
	if( UICommandList->ProcessCommandBindings( InKeyEvent ) )
	{
		// handle the event if a command was processed
		Reply = FReply::Handled();
	}
	return Reply;
}

FReply SPListEditorPanel::OnMouseButtonDown( const FGeometry& /*MyGeometry*/, const FPointerEvent& MouseEvent )
{
	// Perform commands if necessary
	FReply Reply = FReply::Unhandled();
	if( UICommandList->ProcessCommandBindings( MouseEvent ) )
	{
		// handle the event if a command was processed
		Reply = FReply::Handled();
	}
	return Reply;
}

/** Delegate to generate the context menu for the ListView */
TSharedPtr<SWidget> SPListEditorPanel::OnContextMenuOpen() 
{
	FMenuBuilder MenuBuilder(true, UICommandList);

	MenuBuilder.BeginSection("EntryModifications", LOCTEXT("PListContextHeadingElements", "Entry Modifications"));

	MenuBuilder.AddMenuEntry(FPListEditorCommands::Get().MoveUpCommand);

	MenuBuilder.AddMenuEntry(FPListEditorCommands::Get().MoveDownCommand);

	MenuBuilder.AddMenuEntry(FPListEditorCommands::Get().DeleteSelectedCommand);

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("AddOperations", LOCTEXT("PListContextHeadingAdd", "Add Operations"));

	MenuBuilder.AddMenuEntry(FPListEditorCommands::Get().AddStringCommand);

	MenuBuilder.AddMenuEntry(FPListEditorCommands::Get().AddBooleanCommand);

	MenuBuilder.AddMenuEntry(FPListEditorCommands::Get().AddArrayCommand);

	MenuBuilder.AddMenuEntry(FPListEditorCommands::Get().AddDictionaryCommand);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

/** Delegate to handle when a text option is chosen from right-click menu */
void SPListEditorPanel::OnPopupTextChosen(const FString& ChosenText)
{
	FSlateApplication::Get().DismissAllMenus();
}

EActiveTimerReturnType SPListEditorPanel::DisplayDeferredNotifications( double InCurrentTime, float InDeltaTime )
{
	if ( --FramesToSkip == 0 )
	{
		for ( int32 i = 0; i < QueuedNotifications.Num(); ++i )
		{
			if ( QueuedNotifications[i].NotificationType() == NTF_Normal )
			{
				NotificationListPtr->AddNotification( FNotificationInfo( QueuedNotifications[i].Notification() ) );
			}
			else if ( QueuedNotifications[i].NotificationType() == NTF_Success )
			{
				FNotificationInfo info( QueuedNotifications[i].Notification() );
				TWeakPtr<SNotificationItem> PendingProgressPtr = NotificationListPtr->AddNotification( info );
				PendingProgressPtr.Pin()->SetCompletionState( SNotificationItem::CS_Success );
			}
			else // if(QueuedNotifications[i].NotificationType() == NTF_Failed)
			{
				FNotificationInfo info( QueuedNotifications[i].Notification() );
				TWeakPtr<SNotificationItem> PendingProgressPtr = NotificationListPtr->AddNotification( info );
				PendingProgressPtr.Pin()->SetCompletionState( SNotificationItem::CS_Fail );
			}
		}

		QueuedNotifications.Empty();

		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

/** Helper function to display notifications in the current tab */
void SPListEditorPanel::DisplayNotification(const FText& ToDisplay, const ENTF_Types NotificationType)
{
	// Register the active timer if it isn't already
	if ( FramesToSkip == 0 )
	{
		RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SPListEditorPanel::DisplayDeferredNotifications ) );
	}

	QueuedNotifications.Add( FQueuedNotification( ToDisplay, NotificationType ) );
	FramesToSkip = 15; // Hack to get notifications to always show full animations (would break if displaying >1 notification within 'FramesToSkip' frames)
}

/** Marks the widget as being dirty, forcing a prompt on saving before some actions */
void SPListEditorPanel::MarkDirty()
{
	bDirty = true;

	// Show a little token representing dirty
	FileNameWidget->SetText(FText::FromString(FString(TEXT("* ")) + InOutLastPath));
}

/** Clears the dirty flag, internal use only */
void SPListEditorPanel::ClearDirty()
{
	bDirty = false;

	// Clear token representing dirty
	FileNameWidget->SetText(FText::FromString(InOutLastPath));
}

/** Delegate to check if the SearchBar is/should be enabled */
bool SPListEditorPanel::IsSearchBarEnabled() const
{
	return PListNodes.Num() > 0;
}

/** Delegate to handle when the user changes filter text */
void SPListEditorPanel::OnFilterTextChanged( const FText& InFilterText )
{
	// Let file know that the filter changed, which will let all children know the text changed
	if(PListNodes.Num())
	{
		check(PListNodes.Num() == 1);
		PListNodes[0]->OnFilterTextChanged(InFilterText.ToString());
	}
}

#undef LOCTEXT_NAMESPACE
