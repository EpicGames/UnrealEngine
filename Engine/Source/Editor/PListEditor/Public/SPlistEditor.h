// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FArchive;
class ITableRow;
class SNotificationList;
class SSearchBox;
class SWidget;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "PListEditorCommands"

class FXmlFile;
class IPListNode;
class SEditableText;

// Type of notification to spawn
enum ENTF_Types
{
	NTF_Normal,
	NTF_Success,
	NTF_Fail
};

// A queued notification for handling
struct FQueuedNotification
{
public:

	/** Constructs the queued notification */
	FQueuedNotification(const FText& InNotif, const ENTF_Types InNotifType)
		: Notif(InNotif), NotifType(InNotifType)
	{
	}

public:

	/** Retrieves the notification */
	const FText Notification() const
	{
		return Notif;
	}
	/** Retrieves the notification type */
	const ENTF_Types NotificationType() const
	{
		return NotifType;
	}

private:

	/** The stored notification */
	FText Notif;
	/** The stored notificaiton type */
	ENTF_Types NotifType;
};

/** Class for keybindings for the editor */
class FPListEditorCommands : public TCommands<FPListEditorCommands>
{
public:

	/** Register Command group */
	FPListEditorCommands() 
		: TCommands<FPListEditorCommands>("PListEditor", NSLOCTEXT("PListEditor", "PListEditor", "PList Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	/** New Command */
	TSharedPtr<FUICommandInfo> NewCommand;
	/** Open Command */
	TSharedPtr<FUICommandInfo> OpenCommand;
	/** Save Command */
	TSharedPtr<FUICommandInfo> SaveCommand;
	/** SaveAs Command */
	TSharedPtr<FUICommandInfo> SaveAsCommand;
	/** DeleteSelected Command */
	TSharedPtr<FUICommandInfo> DeleteSelectedCommand;
	/** Move Entry up in list command */
	TSharedPtr<FUICommandInfo> MoveUpCommand;
	/** Move Entry down in list command */
	TSharedPtr<FUICommandInfo> MoveDownCommand;
	/** Add Dictionary Command */
	TSharedPtr<FUICommandInfo> AddDictionaryCommand;
	/** Add Array Command */
	TSharedPtr<FUICommandInfo> AddArrayCommand;
	/** Add String Command */
	TSharedPtr<FUICommandInfo> AddStringCommand;
	/** Add Boolean Command */
	TSharedPtr<FUICommandInfo> AddBooleanCommand;

	/** Initialize Commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND( NewCommand, "New", "Creates a new plist file", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::N) );
		UI_COMMAND( OpenCommand, "Open", "Opens an existing plist file", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::O) );
		UI_COMMAND( SaveCommand, "Save", "Saves the current plist file", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::S) );
		UI_COMMAND( SaveAsCommand, "Save As", "Saves the current plist file to a specific location", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::S) );
		UI_COMMAND( DeleteSelectedCommand, "Remove Selected", "Removed the selected entries from the plist", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::R) );
		UI_COMMAND( MoveUpCommand, "Move Up", "Moves the selected entry up within its parent", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::U) );
		UI_COMMAND( MoveDownCommand, "Move Down", "Moves the selected entry down within its parent", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::D) );
		UI_COMMAND( AddDictionaryCommand, "Add Dictionary", "Adds a new dictionary to the selected file or array", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND( AddArrayCommand, "Add Array", "Adds a new array to the selected file, array, or dictionary", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND( AddStringCommand, "Add String", "Adds a new string to the selected file, array, or dictionary", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND( AddBooleanCommand, "Add Boolean", "Adds a new boolean to the selected file, array, or dictionary", EUserInterfaceActionType::Button, FInputChord());
	}
};

class SPListEditorPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPListEditorPanel ) {}
	
	SLATE_END_ARGS()
		
	/** Constructor */
	SPListEditorPanel()
		: UICommandList(new FUICommandList())
	{
	}
	
	// SWidget interface
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	// End of SWidget interface

	/** Construct the widget when opened */
	void Construct( const FArguments& InArgs );

	/** Handles when the tab is trying to be closed (prompt saving if necessary) */
	bool OnTabClose();

private:
	
	/** An internal array holding parsed data from a loaded plist file */
	TArray<TSharedPtr<IPListNode> > PListNodes;
	/** The list widget, needed so we can request refreshes when we change the list's contents */
	TSharedPtr<STreeView<TSharedPtr<IPListNode> > > InternalTree;

	/** The search bar widget */
	TSharedPtr<SSearchBox> SearchBox;

	/** The list of active system messages */
	TSharedPtr<SNotificationList> NotificationListPtr;
	/** A queue of notifications to display on subsequent frames */
	TArray<FQueuedNotification> QueuedNotifications;
	/** How many frames to skip before trying to display a notification */
	int32 FramesToSkip;

	/** The widget that shows the filename */
	TSharedPtr<SEditableText> FileNameWidget;
	/** The last loaded file */
	FString InOutLastPath;
	/** Whether or not a file is currently loaded. Also encompasses when a new file is created */
	bool bFileLoaded;
	/** Whether or not to prompt a save before going through */
	bool bPromptSave;
	/** Whether a new file is created that needs a save location */
	bool bNewFile;
	/** Dirty flag: Anything has been touched in PListNodes */
	bool bDirty;
	/** Flag on whether to prompt for deletions of not */
	bool bPromptDelete;

	/** The list of UI Commands executable */
	TSharedRef<FUICommandList> UICommandList;

private:

	/** Delegate to generate a row for each member of the internal SList */
	TSharedRef<ITableRow> OnGenerateRow( TSharedPtr<IPListNode> InItem, const TSharedRef<STableViewBase>& OwnerTable );
	/** Delegate to get the children of the stored items */
	void OnGetChildren(TSharedPtr<IPListNode> InItem, TArray<TSharedPtr<IPListNode> >& OutItems);

public:

	/** Delegate to add a new array to the plist as a child of the selected node */
	void OnAddArray();
	/** Delegate for New ui command */
	void OnNew();
	/** Delegate to create a new plist when the button is clicked */
	FReply OnNewClicked();
	/** Delegate for Open ui command */
	void OnOpen();
	/** Delegate to open an existing plist when the button is clicked */
	FReply OnOpenClicked();
	/** Delegate for Save ui command */
	void OnSave();
	/** Delegate to save the working plist when the button is clicked */
	FReply OnSaveClicked();
	/** Delegate for SaveAs ui command */
	void OnSaveAs();
	/** Delegate to save the working plist with a specified name when the button is clicked */
	FReply OnSaveAsClicked();
	/** Delegate for DeleteSelected ui command */
	void OnDeleteSelected();
	/** Delegate for MoveUp command */
	void OnMoveUp();
	/** Delegate for MoveDown command */
	void OnMoveDown();
	/** Delegate for adding a dictionary */
	void OnAddDictionary();
	/** Delegate for adding a string */
	void OnAddString();
	/** Delegate for adding a boolean */
	void OnAddBoolean();
	/** Delegate to handle when a text option is chosen from right-click menu */
	void OnPopupTextChosen(const FString& ChosenText);

	/** Delegate to generate the context menu for the ListView */
	TSharedPtr<SWidget> OnContextMenuOpen();
	/** Delegate that determines when the delete selected context button can be clicked */
	bool DetermineDeleteSelectedContext() const;
	/** Delegate that determines when the AddArray button can be clicked in the context menu */
	bool DetermineAddArrayContext() const;
	/** Delegate for determining when MoveUp can be used */
	bool DetermineMoveUpContext() const;
	/** Delegate for determining when MoveDown can be used */
	bool DetermineMoveDownContext() const;
	/** Delegate for determining when AddDictionary can be used */
	bool DetermineAddDictionaryContext() const;
	/** Delegate for determining when AddString can be used */
	bool DetermineAddStringContext() const;
	/** Delegate for determining when AddBoolean can be used */
	bool DetermineAddBooleanContext() const;

	/** Delegate to check if the SearchBar is/should be enabled */
	bool IsSearchBarEnabled() const;
	/** Delegate to handle when the user changes filter text */
	void OnFilterTextChanged( const FText& InFilterText );

public:

	/** Marks the widget as being dirty, forcing a prompt on saving before some actions */
	void MarkDirty();

private:
	/** Displays queued notifications */
	EActiveTimerReturnType DisplayDeferredNotifications( double InCurrentTime, float InDeltaTime );
	/** Helper method to display queued notifications */
	void PerformDisplayNotifications();


	/** Helper method to parse through and load an Xml tree into an internal intermediate format for Slate */
	bool ParseXmlTree(FXmlFile& Doc, FString& OutError);
	/** Helper method to write out the contents of PListNodes using a valid FileWriter */
	void SerializePListNodes(FArchive* Writer);
	/** Helper function to check if PListNodes is valid (every element being valid) */
	bool ValidatePListNodes();
	/** Helper function to display notifications in the current tab */
	void DisplayNotification(const FText& ToDisplay, const ENTF_Types NotificationType = NTF_Normal);
	/** Clears the dirty flag, internal use only */
	void ClearDirty();
	/** Helper method to perform a save prompt. Returns true if the caller can pass the prompt and false if the caller should not continue its routine */
	bool PromptSave();
	/** Helper method to prompt the user to delete element(s) */
	bool PromptDelete();
	/** Helper method to check if a file exists */
	bool CheckFileExists(FString Path);
	/** Helper function to open a file. Does not perform any checks before performing operation (such as bDirty)!! */
	bool OpenFile(FString FilePath);
	/** Helper method to bind commands */
	void BindCommands();
	/** Helper function to search through nodes to find a specific node's parent */
	bool FindParent(const TSharedPtr<IPListNode>& InChildNode, TSharedPtr<IPListNode>& OutFoundNode) const;
};

#undef LOCTEXT_NAMESPACE
